[CoreRedirects]

+ClassRedirects=(OldName="VRTool",NewName="/Script/VPUtilitiesEditor.VRTool")
+ClassRedirects=(OldName="VPSettings",NewName="/Script/VPSettings.VPSettings")
+ClassRedirects=(OldName="VirtualProductionRolesSubsystem",NewName="/Script/VPRoles.VirtualProductionRolesSubsystem")
+ClassRedirects=(OldName="/Script/VPUtilities.VPTimecodeCustomTimeStep",NewName="/Script/TimeManagement.TimecodeCustomTimeStep")
+FunctionRedirects=(OldName="VPViewportTickableActorBase.LockLocation",NewName="VPViewportTickableActorBase.EditorLockLocation")

; 5.2 > 5.3
+PropertyRedirects=(OldName="VPFullScreenUserWidget_PostProcess.PostProcessMaterial",NewName="VPFullScreenUserWidget_PostProcessBase.PostProcessMaterial")
+PropertyRedirects=(OldName="VPFullScreenUserWidget_PostProcess.PostProcessTintColorAndOpacity",NewName="VPFullScreenUserWidget_PostProcessBase.PostProcessTintColorAndOpacity")
+PropertyRedirects=(OldName="VPFullScreenUserWidget_PostProcess.PostProcessOpacityFromTexture",NewName="VPFullScreenUserWidget_PostProcessBase.PostProcessOpacityFromTexture")
+PropertyRedirects=(OldName="VPFullScreenUserWidget_PostProcess.bUseWidgetDrawSize",NewName="VPFullScreenUserWidget_PostProcessBase.bUseWidgetDrawSize")
+PropertyRedirects=(OldName="VPFullScreenUserWidget_PostProcess.WidgetDrawSize",NewName="VPFullScreenUserWidget_PostProcessBase.WidgetDrawSize")
+PropertyRedirects=(OldName="VPFullScreenUserWidget_PostProcess.bWindowFocusable",NewName="VPFullScreenUserWidget_PostProcessBase.bWindowFocusable")
+PropertyRedirects=(OldName="VPFullScreenUserWidget_PostProcess.WindowVisibility",NewName="VPFullScreenUserWidget_PostProcessBase.WindowVisibility")
+PropertyRedirects=(OldName="VPFullScreenUserWidget_PostProcess.bReceiveHardwareInput",NewName="FVPFullScreenUserWidget_PostProcessBase.bReceiveHardwareInput")
+PropertyRedirects=(OldName="VPFullScreenUserWidget_PostProcess.RenderTargetBackgroundColor",NewName="VPFullScreenUserWidget_PostProcessBase.RenderTargetBackgroundColor")
+PropertyRedirects=(OldName="VPFullScreenUserWidget_PostProcess.RenderTargetBlendMode",NewName="FVPFullScreenUserWidget_PostProcessBase.RenderTargetBlendMode")
+PropertyRedirects=(OldName="VPFullScreenUserWidget.WidgetRenderTarget",NewName="VPFullScreenUserWidget_PostProcessBase.WidgetRenderTarget")
+PropertyRedirects=(OldName="VPFullScreenUserWidget.PostProcessDisplayType",NewName="VPFullScreenUserWidget.PostProcessDisplayTypeWithBlendMaterial")
; This creates a conflict with the redirect in DefaultViewportWidgetOverlay.ini
;+EnumRedirects=(OldName="/Script/VPUtilities.EVPWidgetDisplayType", NewName="/Script/VPUtilities.EVPWidgetDisplayType", ValueChanges=(("PostProcess", "PostProcessWithBlendMaterial")))

; 5.5 > 5.6
+PackageRedirects=(OldName="/VirtualProductionUtilities/Bookmark/BP_VPBookmarkActor",NewName="/VirtualProductionUtilities/DEPRECATED/BP_VPBookmarkActor")

[/Script/VPUtilities.VPBookmarkSettings]

BookmarkMeshPath=/VirtualProductionUtilities/Bookmark/BookmarkVPMesh.BookmarkVPMesh
BookmarkMaterialPath=/VirtualProductionUtilities/Bookmark/MI_BookmarkVPNoUser.MI_BookmarkVPNoUser
BookmarkSplineMeshPath=/Engine/VREditor/FloatingText/LineSegmentCylinder.LineSegmentCylinder
BookmarkSplineMeshMaterialPath=/Engine/VREditor/LaserPointer/LaserPointerMaterialInst.LaserPointerMaterialInst
BookmarkLabelMaterialPath=/VirtualProductionUtilities/Bookmark/CameraFacingTextMaterial.CameraFacingTextMaterial

[/Script/VPUtilitiesEditor.VPUtilitiesEditorSettings]
ScoutingSubsystemEditorUtilityClassPath=/Script/VPUtilitiesEditor.VPScoutingSubsystemHelpersBase
GestureManagerEditorUtilityClassPath=/Script/VPUtilitiesEditor.VPScoutingSubsystemGestureManagerBase
