// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPBookmark.h"
#include "IVPBookmarkProvider.h"
#include "VPBookmarkLifecycleDelegates.h"

#include "Engine/Engine.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"


#define LOCTEXT_NAMESPACE "VPBookmark"


FVPBookmarkViewportData::FVPBookmarkViewportData()
	: JumpToOffsetLocation(FVector::ZeroVector)
	, LookRotation(FRotator::ZeroRotator)
	, OrthoZoom(0.f)
	, bFlattenRotation(false)
{
}


void UVPBookmark::SetActive(bool bInActive)
{
	if (bIsActive != bInActive)
	{
		bIsActive = bInActive;
		if (AActor* OwnedActorPtr = OwnedActor.Get())
		{
			if (OwnedActorPtr->GetClass()->ImplementsInterface(UVPBookmarkProvider::StaticClass()))
			{
				IVPBookmarkProvider::Execute_OnBookmarkActivation(OwnedActorPtr, this, bIsActive);
			}
		}
	}
}


int32 UVPBookmark::GetBookmarkIndex() const
{
	int32 Result = INDEX_NONE;
	if (UWorld* World = GetWorld())
	{
		if (AWorldSettings* WorldSettings = World->GetWorldSettings())
		{
			const TArray<UBookmarkBase*>& Bookmarks = WorldSettings->GetBookmarks();
			Bookmarks.Find(const_cast<UVPBookmark*>(this), Result);
		}
	}
	return Result;
}


AActor* UVPBookmark::GetAssociatedBookmarkActor() const
{
	return OwnedActor.Get();
}


FText UVPBookmark::GetDisplayName() const
{
	return FText::Format(LOCTEXT("BookmarkDisplayNameFormat", "({0}) {1}")
		, FText::AsNumber(GetBookmarkIndex())
		, FText::FromString(CreationContext.DisplayName)
		);
}


void UVPBookmark::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
#if WITH_EDITOR
		if (GEngine)
		{
			// Ask the engine when the actor is loaded or destroyed
			if (!OwnedActor.IsValid())
			{
				OnLevelActorAddedHandle = GEngine->OnLevelActorAdded().AddUObject(this, &UVPBookmark::OnLevelActorAdded);

			}
			OnLevelActorDeletedHandle = GEngine->OnLevelActorDeleted().AddUObject(this, &UVPBookmark::OnLevelActorDeleted);
		}
#endif

		FVPBookmarkLifecycleDelegates::GetOnBookmarkCreated().Broadcast(this);
	}
}


void UVPBookmark::PostLoad()
{
	Super::PostLoad();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		// Is there an actor attached to the bookmark, if not remove it (note that it can live in a sub level and may not be loaded yet)
		if (OwnedActor.IsNull())
		{
			RemoveBookmark();
		}
	}
}


void UVPBookmark::BeginDestroy()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		FVPBookmarkLifecycleDelegates::GetOnBookmarkDestroyed().Broadcast(this);

#if WITH_EDITOR
		if (GEngine)
		{
			GEngine->OnLevelActorAdded().Remove(OnLevelActorAddedHandle);
			GEngine->OnLevelActorDeleted().Remove(OnLevelActorDeletedHandle);
		}
#endif
	}

	Super::BeginDestroy();
}


void UVPBookmark::OnCleared()
{
	if (bIsActive)
	{
		SetActive(false);
	}
	if (AActor* OwnedActorPtr = OwnedActor.Get())
	{
		if (!OwnedActorPtr->IsPendingKillPending() && !OwnedActorPtr->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed))
		{
			if (UWorld* World = OwnedActorPtr->GetWorld())
			{
				if (World->IsEditorWorld() && !World->IsPlayInEditor())
				{
					World->EditorDestroyActor(OwnedActorPtr, true);
				}
				else
				{
					World->DestroyActor(OwnedActorPtr);
				}
			}
		}
	}

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		FVPBookmarkLifecycleDelegates::GetOnBookmarkCleared().Broadcast(this);
	}
}


void UVPBookmark::BookmarkChanged(AActor* OwnedActorPtr)
{
	check(OwnedActorPtr);
	if (OwnedActorPtr->GetClass()->ImplementsInterface(UVPBookmarkProvider::StaticClass()))
	{
		IVPBookmarkProvider::Execute_OnBookmarkChanged(OwnedActorPtr, this);
	}
}


void UVPBookmark::RemoveBookmark()
{
	if (UWorld* World = GetWorld())
	{
		if (AWorldSettings* WorldSettings = World->GetWorldSettings())
		{
			int32 BookmarkIndex = INDEX_NONE;
			const TArray<UBookmarkBase*>& Bookmarks = WorldSettings->GetBookmarks();
			if (Bookmarks.Find(const_cast<UVPBookmark*>(this), BookmarkIndex))
			{
				WorldSettings->ClearBookmark(BookmarkIndex);
			}
			else
			{
				MarkAsGarbage();
			}
		}
	}
}


void UVPBookmark::OnLevelActorAdded(AActor* NewActor)
{
#if WITH_EDITOR
	if (NewActor == OwnedActor.Get())
	{
		BookmarkChanged(NewActor);

		check(GEngine);
		GEngine->OnLevelActorAdded().Remove(OnLevelActorAddedHandle);
		OnLevelActorAddedHandle.Reset();
	}
#endif
}


void UVPBookmark::OnLevelActorDeleted(AActor* DeletedActor)
{
#if WITH_EDITOR
	if (DeletedActor == OwnedActor.Get())
	{
		RemoveBookmark();

		check(GEngine);
		GEngine->OnLevelActorDeleted().Remove(OnLevelActorDeletedHandle);
		OnLevelActorDeletedHandle.Reset();
	}
#endif
}

#undef LOCTEXT_NAMESPACE
