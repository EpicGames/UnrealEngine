// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPBookmarkBlueprintLibrary.h"
#include "VPBookmark.h"
#include "VPBookmarkModule.h"

#include "Engine/Engine.h"
#include "EngineUtils.h"
#include "GameFramework/Actor.h"
#include "UObject/UObjectHash.h"
#include "IVPBookmarkProvider.h"
#include "VPBookmarkModule.h"

UVPBookmark* UVPBookmarkBlueprintLibrary::FindVPBookmark(const AActor* Actor)
{
	UVPBookmark* Result = nullptr;
	if (Actor)
	{
		if (UWorld* World = Actor->GetWorld())
		{
			if (AWorldSettings* WorldSettings = World->GetWorldSettings())
			{
				const TArray<UBookmarkBase*>& Bookmarks = WorldSettings->GetBookmarks();
				for (UBookmarkBase* Base : Bookmarks)
				{
					if (UVPBookmark* VPBookmark = Cast<UVPBookmark>(Base))
					{
						if (VPBookmark->OwnedActor.Get() == Actor)
						{
							Result = VPBookmark;
							break;
						}
					}
				}
			}
		}
	}
	return Result;
}


void UVPBookmarkBlueprintLibrary::GetAllVPBookmarkActors(const UObject* WorldContextObject, TArray<AActor*>& OutActors)
{
	OutActors.Reset();

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World)
	{
		AWorldSettings* WorldSettings = World->GetWorldSettings();
		if (WorldSettings != nullptr)
		{
			const TArray<UBookmarkBase*>& Bookmarks = WorldSettings->GetBookmarks();
			OutActors.Reset(Bookmarks.Num());
			for (UBookmarkBase* Base : Bookmarks)
			{
				if (UVPBookmark* Bookmark = Cast<UVPBookmark>(Base))
				{
					if (AActor* Actor = Bookmark->OwnedActor.Get())
					{
						if (IsValid(Bookmark) && IsValid(Actor))
						{
							OutActors.Add(Actor);
						}
					}
				}
			}
		}
	}
}


void UVPBookmarkBlueprintLibrary::GetAllVPBookmark(const UObject* WorldContextObject, TArray<UVPBookmark*>& OutBookmarks)
{
	OutBookmarks.Reset();

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World)
	{
		AWorldSettings* WorldSettings = World->GetWorldSettings();
		if (WorldSettings != nullptr)
		{
			const TArray<UBookmarkBase*>& Bookmarks = WorldSettings->GetBookmarks();
			OutBookmarks.Reset(Bookmarks.Num());
			for (UBookmarkBase* Base : Bookmarks)
			{
				if (UVPBookmark* Bookmark = Cast<UVPBookmark>(Base))
				{
					if (IsValid(Bookmark))
					{
						OutBookmarks.Add(Bookmark);
					}
				}
			}
		}
	}
}

// @TODO: This is just a rudimentary implementation that doesn't properly handle gaps
bool UVPBookmarkBlueprintLibrary::CreateVPBookmarkName(AActor* Bookmark, const FString& FormatString, FString& GeneratedNumber, FString& GeneratedLetter)
{
	if (Bookmark == nullptr)
	{
		UE_LOGF(LogVPBookmark, Error, "UVPBookmarkBlueprintLibrary::CreateVPBookmarkName - No actor provided");
		return false;
	}

	if (!Bookmark->GetClass()->ImplementsInterface(UVPBookmarkProvider::StaticClass()))
	{
		UE_LOGF(LogVPBookmark, Warning, "UVPBookmarkBlueprintLibrary::CreateVPBookmarkName - Actor '%ls' doesn't implement IVPBookmarkProvider", *(Bookmark->GetClass()->GetName()));
		return false;
	}

	// Set display name of the new bookmark to read "Bookmark 1", "Bookmark 2" etc. 
	// @todo: To make this work in a concert session, we have to collect all actors in the scene and see which number to use next - we don't have the concept of 
	// a shared static variable or "Concert scratchpad" yet to keep track.
	static const int32 InitialBookmarkIndex = 0;
	static FString NoPlaceholders(FormatString);
	NoPlaceholders = NoPlaceholders.Replace(TEXT("%l"), TEXT(""));
	NoPlaceholders = NoPlaceholders.Replace(TEXT("%n"), TEXT(""));
	NoPlaceholders.RemoveSpacesInline();

	if (!NoPlaceholders.IsEmpty())
	{
		int32 HighestBookmarkIndex = InitialBookmarkIndex;
		int32 NumBookmarks = 0;

		// Iterator over all actors (and then check class inside the loop) because we often handle classes only defined in BP with no native parent
		for (TActorIterator<AActor> It(Bookmark->GetWorld()); It; ++It)
		{
			const AActor* CurrentBookmark = *It;
			if (CurrentBookmark != Bookmark && CurrentBookmark->GetClass() == Bookmark->GetClass() && !CurrentBookmark->IsTemplate())
			{
				++NumBookmarks;

				/*int32 CurrentBookmarkIndex = InitialBookmarkIndex;
				if (ExtractBookmarkIndex(CurrentBookmark, CurrentBookmarkIndex) && CurrentBookmarkIndex > HighestBookmarkIndex)
				{
					HighestBookmarkIndex = CurrentBookmarkIndex;
				}*/
			}
		}
		GeneratedNumber = *FString::FromInt(++NumBookmarks);
		GeneratedLetter = *FString::Chr(NumBookmarks + 64);
		FString NewLabel = FormatString.Replace(TEXT("%n"), *GeneratedNumber);
		NewLabel = NewLabel.Replace(TEXT("%l"), *GeneratedLetter);

#if WITH_EDITOR
		Bookmark->SetActorLabel(NewLabel, false); // UED outliner label
#endif //WITH_EDITOR
	
		return true;
	}
	return false;
}
