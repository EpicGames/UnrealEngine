// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPBookmarkLifecycleDelegates.h"


FVPBookmarkLifecycleDelegates::FVPBookmarkLifecycleDelegate& FVPBookmarkLifecycleDelegates::GetOnBookmarkCreated()
{
	static FVPBookmarkLifecycleDelegate OnBookmarkCreated;
	return OnBookmarkCreated;
}


FVPBookmarkLifecycleDelegates::FVPBookmarkLifecycleDelegate& FVPBookmarkLifecycleDelegates::GetOnBookmarkDestroyed()
{
	static FVPBookmarkLifecycleDelegate OnBookmarkDestroyed;
	return OnBookmarkDestroyed;
}


FVPBookmarkLifecycleDelegates::FVPBookmarkLifecycleDelegate& FVPBookmarkLifecycleDelegates::GetOnBookmarkCleared()
{
	static FVPBookmarkLifecycleDelegate OnBookmarkCleared;
	return OnBookmarkCleared;
}
