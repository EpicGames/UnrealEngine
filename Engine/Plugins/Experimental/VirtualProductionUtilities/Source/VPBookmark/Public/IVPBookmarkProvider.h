// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "IVPBookmarkProvider.generated.h"


class UVPBookmark;


UINTERFACE(BlueprintType)
class VPBOOKMARK_API UVPBookmarkProvider : public UInterface
{
	GENERATED_BODY()
};


class VPBOOKMARK_API IVPBookmarkProvider : public IInterface
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintNativeEvent, CallInEditor, Category = "Bookmarks")
	void OnBookmarkActivation(UVPBookmark* Bookmark, bool bActivate);

	UFUNCTION(BlueprintNativeEvent, CallInEditor, Category = "Bookmarks")
	void OnBookmarkChanged(UVPBookmark* Bookmark);

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, CallInEditor, Category = "Bookmarks")
	void UpdateBookmarkSplineMeshIndicator();

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, CallInEditor, Category = "Bookmarks")
	void HideBookmarkSplineMeshIndicator();

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, CallInEditor, Category = "Bookmarks")
	void GenerateBookmarkName();
};
