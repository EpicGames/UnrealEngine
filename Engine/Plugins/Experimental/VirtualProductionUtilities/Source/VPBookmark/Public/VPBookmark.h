// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/BookmarkBase.h"
#include "VPBookmarkContext.h"
#include "VPBookmark.generated.h"


class AActor;


USTRUCT(BlueprintType)
struct VPBOOKMARK_API FVPBookmarkViewportData
{
	GENERATED_BODY()

public:
	FVPBookmarkViewportData();

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Bookmarks")
	FVector JumpToOffsetLocation;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Bookmarks")
	FRotator LookRotation;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Bookmarks")
	float OrthoZoom;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Bookmarks")
	bool bFlattenRotation;
};


USTRUCT()
struct VPBOOKMARK_API FVPBookmarkJumpToSettings : public FBookmarkBaseJumpToSettings
{
	GENERATED_BODY()
};


UCLASS(BlueprintType, Category = Bookmark)
class VPBOOKMARK_API UVPBookmark : public UBookmarkBase
{
	GENERATED_BODY()

private:
	UPROPERTY(DuplicateTransient, Transient)
	bool bIsActive;

public:
	UPROPERTY()
	TLazyObjectPtr<AActor> OwnedActor;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Bookmarks")
	FVPBookmarkCreationContext CreationContext;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Bookmarks")
	FVPBookmarkViewportData CachedViewportData;

public:
	UFUNCTION(BlueprintCallable, Category = "Bookmarks")
	bool IsActive() const { return bIsActive; }

	void SetActive(bool bInActive);

	UFUNCTION(BlueprintCallable, Category = "Bookmarks")
	int32 GetBookmarkIndex() const;

	UFUNCTION(BlueprintCallable, Category = "Bookmarks")
	AActor* GetAssociatedBookmarkActor() const;

	UFUNCTION(BlueprintCallable, Category = "Bookmarks")
	FText GetDisplayName() const;

public:

	//~ Begin UObject Interface
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
	//~ End UObject Interface

	//~ Begin UBookmarkBase Interface
	virtual void OnCleared() override;
	//~ End UBookmarkBase Interface

private:
	void BookmarkChanged(AActor* OwnerPtr);
	void RemoveBookmark();

	void OnLevelActorAdded(AActor* NewActor);
	void OnLevelActorDeleted(AActor* DeletedActor);

#if WITH_EDITORONLY_DATA
	FDelegateHandle OnLevelActorAddedHandle;
	FDelegateHandle OnLevelActorDeletedHandle;
#endif
};
