// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "VPBookmarkBlueprintLibrary.generated.h"


class AActor;
class UVPBookmark;


UCLASS(meta=(ScriptName="VPBookmarkLibrary"))
class VPBOOKMARK_API UVPBookmarkBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Bookmarks")
	static UVPBookmark* FindVPBookmark(const AActor* Actor);

	UFUNCTION(BlueprintCallable, Category = "Bookmarks", meta = (WorldContext = "WorldContextObject"))
	static void GetAllVPBookmarkActors(const UObject* WorldContextObject, TArray<AActor*>& OutActors);

	UFUNCTION(BlueprintCallable, Category = "Bookmarks", meta = (WorldContext = "WorldContextObject"))
	static void GetAllVPBookmark(const UObject* WorldContextObject, TArray<UVPBookmark*>& OutBookmarks);

	UFUNCTION(BlueprintCallable, Category = "Bookmarks")
	static bool CreateVPBookmarkName(AActor* Bookmark, const FString& FormatString, FString& GeneratedNumber, FString& GeneratedLetter);
};
