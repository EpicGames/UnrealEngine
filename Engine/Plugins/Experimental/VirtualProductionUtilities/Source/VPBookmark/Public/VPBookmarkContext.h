// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "VPBookmarkContext.generated.h"


USTRUCT(BlueprintType)
struct VPBOOKMARK_API FVPBookmarkCreationContext
{
	GENERATED_BODY()

public:
	/** Username of the Concert client who is creating this bookmark. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Bookmarks")
	FString ConcertCreator;

	/** Category of the Bookmark */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Bookmarks")
	FName CategoryName;

	/** Name of the Bookmark */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Bookmarks")
	FString DisplayName;
};

