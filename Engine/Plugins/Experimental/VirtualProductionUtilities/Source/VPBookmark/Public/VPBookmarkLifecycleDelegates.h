// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


class UVPBookmark;


class VPBOOKMARK_API FVPBookmarkLifecycleDelegates
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FVPBookmarkLifecycleDelegate, UVPBookmark*);

	static FVPBookmarkLifecycleDelegate& GetOnBookmarkCreated();
	static FVPBookmarkLifecycleDelegate& GetOnBookmarkDestroyed();
	static FVPBookmarkLifecycleDelegate& GetOnBookmarkCleared();
};
