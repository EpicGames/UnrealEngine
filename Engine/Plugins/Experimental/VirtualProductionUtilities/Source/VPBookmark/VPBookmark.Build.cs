// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VPBookmark : ModuleRules
{
	public VPBookmark(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{

			}
		);
	}
}
