// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVPBookmarkListView.h"

#include "Bookmarks/IBookmarkTypeTools.h"
#include "Editor.h"
#include "Engine/BookmarkBase.h"
#include "Styling/AppStyle.h"
#include "LevelEditorViewport.h"
#include "VPBookmarkBlueprintLibrary.h"
#include "VPBookmarkEditorBlueprintLibrary.h"
#include "VPBookmarkLifecycleDelegates.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "VPBookmarkListView"


namespace VPBookmarkList
{
	static FEditorViewportClient* BookmarkUtilsGetUsableViewportClient()
	{
		return GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient :
			GLastKeyLevelEditingViewportClient ? GLastKeyLevelEditingViewportClient :
			nullptr;
	}


	class SVPBookmarkCategoryListView : public SCompoundWidget
	{
	private:
		using Super = SCompoundWidget;

	public:
		SLATE_BEGIN_ARGS(SVPBookmarkCategoryListView) { }
		SLATE_ARGUMENT(FName, Category)
		SLATE_END_ARGS()


		void Construct(const FArguments& InArgs)
		{
			Category = InArgs._Category;

			ChildSlot
			[
				SAssignNew(ExpandableArea, SExpandableArea)
				.BorderImage(this, &SVPBookmarkCategoryListView::GetBackgroundImage)
				.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
				.BodyBorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.BodyBorderBackgroundColor(FLinearColor::White)
				.HeaderContent()
				[
					SNew(STextBlock)
					.Text(FText::FromName(Category))
					.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
					.ShadowOffset(FVector2D(1.0f, 1.0f))
				]
				.BodyContent()
				[
					SAssignNew(BookmarkListView, SListView<TWeakObjectPtr<UVPBookmark>>)
					.ListItemsSource(&Bookmarks)
					.SelectionMode(ESelectionMode::Single)
					.OnGenerateRow(this, &SVPBookmarkCategoryListView::GenerateBookmarkRow)
					.OnSelectionChanged(this, &SVPBookmarkCategoryListView::OnBookmarkSelected)
					.AllowOverscroll(EAllowOverscroll::No)
				]
			];
		}


	private:
		TSharedRef<ITableRow> GenerateBookmarkRow(TWeakObjectPtr<UVPBookmark> Bookmark, const TSharedRef<STableViewBase>& TableView)
		{
			const FText DisplayName = Bookmark.IsValid() ? Bookmark->GetDisplayName() : LOCTEXT("InvalidBookmark", "<Invalid>");
			return SNew(STableRow<TWeakObjectPtr<UVPBookmark>>, TableView)
				.Content()
				[
					SNew(STextBlock)
					.Text(DisplayName)
				];
		}


		const FSlateBrush* GetBackgroundImage() const
		{
			if (IsHovered())
			{
				return (ExpandableArea.IsValid() && ExpandableArea->IsExpanded()) ? FAppStyle::GetBrush("DetailsView.CategoryTop_Hovered") : FAppStyle::GetBrush("DetailsView.CollapsedCategory_Hovered");
			}
			else
			{
				return (ExpandableArea.IsValid() && ExpandableArea->IsExpanded()) ? FAppStyle::GetBrush("DetailsView.CategoryTop") : FAppStyle::GetBrush("DetailsView.CollapsedCategory");
			}
		}


		void OnBookmarkSelected(TWeakObjectPtr<UVPBookmark> Selected, ESelectInfo::Type SelectionType)
		{
			TSharedPtr<SVPBookmarkListView> OwnerShrPtr = OwnerBookmarkListView.Pin();
			if (OwnerShrPtr.IsValid())
			{
				OwnerShrPtr->OnBookmarkSelected(SharedThis<SVPBookmarkCategoryListView>(this), Selected, SelectionType);
			}
		}


	public:
		FName Category;
		TSharedPtr<SExpandableArea> ExpandableArea;
		TSharedPtr<SListView<TWeakObjectPtr<UVPBookmark>>> BookmarkListView;
		TArray<TWeakObjectPtr<UVPBookmark>> Bookmarks;
		TWeakPtr<SVPBookmarkListView> OwnerBookmarkListView;
	};
}


SVPBookmarkListView::~SVPBookmarkListView()
{
	FVPBookmarkLifecycleDelegates::GetOnBookmarkCleared().RemoveAll(this);
	FVPBookmarkLifecycleDelegates::GetOnBookmarkDestroyed().RemoveAll(this);
	FVPBookmarkLifecycleDelegates::GetOnBookmarkCreated().RemoveAll(this);
	FEditorDelegates::MapChange.RemoveAll(this);
}


void SVPBookmarkListView::Construct(const FArguments& InArgs)
{
	bInSelection = false;

	FEditorDelegates::MapChange.AddSP(this, &SVPBookmarkListView::OnMapChanged);
	FVPBookmarkLifecycleDelegates::GetOnBookmarkCreated().AddSP(this, &SVPBookmarkListView::OnBookmarkListModified);
	FVPBookmarkLifecycleDelegates::GetOnBookmarkDestroyed().AddSP(this, &SVPBookmarkListView::OnBookmarkListModified);
	FVPBookmarkLifecycleDelegates::GetOnBookmarkCleared().AddSP(this, &SVPBookmarkListView::OnBookmarkListModified);

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(0.0f)
		[
			SAssignNew(BookmarkCategoryContainer, SScrollBox)
		]
	];

	PopulateBookmarks();
}


void SVPBookmarkListView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bBookmarkListViewDirty)
	{
		PopulateBookmarks();
	}

	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}


void SVPBookmarkListView::OnMapChanged(uint32 MapChangeFlags)
{
	bBookmarkListViewDirty = true;
}


void SVPBookmarkListView::PopulateBookmarks()
{
	bBookmarkListViewDirty = false;

	TArray<TSharedPtr<VPBookmarkList::SVPBookmarkCategoryListView>> OldBookmarkCategories = BookmarkCategories;
	for (TSharedPtr<VPBookmarkList::SVPBookmarkCategoryListView> Category : BookmarkCategories)
	{
		Category->Bookmarks.Reset();
	}

	if (FEditorViewportClient* Client = VPBookmarkList::BookmarkUtilsGetUsableViewportClient())
	{
		if (UWorld* World = Client->GetWorld())
		{
			TArray<UVPBookmark*> Components;
			UVPBookmarkBlueprintLibrary::GetAllVPBookmark(World, Components);

			// Is it the selected Component
			if (UVPBookmark* SelectedPtr = SelectedBookmark.Get())
			{
				if (!Components.Contains(SelectedPtr))
				{
					GEditor->SelectActor(SelectedPtr->OwnedActor.Get(), /*bSelected=*/false, /*bNotify=*/false);
					SelectedBookmark.Reset();
				}
			}

			for(UVPBookmark* Component : Components)
			{
				// Find the Bookmark category list view
				bool bFound = false;
				for(TSharedPtr<VPBookmarkList::SVPBookmarkCategoryListView> Category : BookmarkCategories)
				{
					if (Category->Category == Component->CreationContext.CategoryName)
					{
						Category->Bookmarks.Emplace(Component);
						bFound = true;
						OldBookmarkCategories.RemoveSwap(Category);
						break;
					}
				}

				// Create a category
				if (!bFound)
				{
					TSharedPtr<VPBookmarkList::SVPBookmarkCategoryListView> Category;
					BookmarkCategoryContainer->AddSlot()
					[
						SAssignNew(Category, VPBookmarkList::SVPBookmarkCategoryListView)
							.Category(Component->CreationContext.CategoryName)
					];
					Category->OwnerBookmarkListView = SharedThis<SVPBookmarkListView>(this);
					BookmarkCategories.Add(Category);

					Category->Bookmarks.Emplace(Component);
				}
			}
		}
	}

	for (TSharedPtr<VPBookmarkList::SVPBookmarkCategoryListView> Category : OldBookmarkCategories)
	{
		BookmarkCategories.Remove(Category);
		BookmarkCategoryContainer->RemoveSlot(Category.ToSharedRef());
	}
}


void SVPBookmarkListView::OnBookmarkListModified(UVPBookmark* Bookmark)
{
	bBookmarkListViewDirty = true;
}


void SVPBookmarkListView::OnBookmarkSelected(TSharedPtr<VPBookmarkList::SVPBookmarkCategoryListView> BookmarkCategory, TWeakObjectPtr<UVPBookmark> Selected, ESelectInfo::Type SelectionType)
{
	if (bInSelection)
	{
		return;
	}
	TGuardValue<bool> Tmp(bInSelection, true);

	for (TSharedPtr<VPBookmarkList::SVPBookmarkCategoryListView> Category : BookmarkCategories)
	{
		if (Category != BookmarkCategory)
		{
			Category->BookmarkListView->ClearSelection();
		}
	}

	if (UVPBookmark* SelectedPtr = SelectedBookmark.Get())
	{
		GEditor->SelectActor(SelectedPtr->OwnedActor.Get(), /*bSelected=*/false, /*bNotify=*/false);
	}

	if (UVPBookmark* SelectedPtr = Selected.Get())
	{
		GEditor->SelectActor(SelectedPtr->OwnedActor.Get(), /*bSelected=*/true, /*bNotify=*/true, /*bSelectEvenIfHidden=*/true);
		IBookmarkTypeTools::Get().JumpToBookmark(SelectedPtr->GetBookmarkIndex(), nullptr, VPBookmarkList::BookmarkUtilsGetUsableViewportClient());
	}

	SelectedBookmark = Selected;
}


TWeakObjectPtr<UVPBookmark> SVPBookmarkListView::GetSelectedBookmark()
{
	return SelectedBookmark;
}


#undef LOCTEXT_NAMESPACE
