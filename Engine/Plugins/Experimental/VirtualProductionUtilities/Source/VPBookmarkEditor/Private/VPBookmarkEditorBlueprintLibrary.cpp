// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPBookmarkEditorBlueprintLibrary.h"
#include "IVPBookmarkProvider.h"
#include "VPBookmark.h"
#include "VPBookmarkEditorModule.h"
#include "VPBookmarkContext.h"
#include "VPBookmarkTypeActions.h"

#include "GameFramework/Actor.h"
#include "EngineUtils.h"
#include "UObject/UObjectHash.h"

#include "Bookmarks/IBookmarkTypeTools.h"
#include "Editor.h"
#include "EditorViewportClient.h"
#include "LevelEditorViewport.h"
#include "IVREditorModule.h"
#include "ViewportWorldInteraction.h"


namespace BookmarkLibrary
{
	static FLevelEditorViewportClient* GetUsableViewportClient()
	{
		return GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient :
			GLastKeyLevelEditingViewportClient ? GLastKeyLevelEditingViewportClient :
			nullptr;
	}
}


bool UVPBookmarkEditorBlueprintLibrary::JumpToBookmarkInLevelEditor(const UVPBookmark* Bookmark)
{
	if (Bookmark)
	{
		return JumpToBookmarkInLevelEditorByIndex(Bookmark->GetBookmarkIndex());
	}
	else
	{
		UE_LOGF(LogVPBookmarkEditor, Warning, "VPBookmarkEditorLibrary::ActivateBookmarkInLevelEditor - Invalid component");
	}

	return false;
}


bool UVPBookmarkEditorBlueprintLibrary::JumpToBookmarkInLevelEditorByIndex(const int32 BookmarkIndex)
{
	FEditorViewportClient* Client = BookmarkLibrary::GetUsableViewportClient();
	if (Client == nullptr)
	{
		UE_LOGF(LogVPBookmarkEditor, Warning, "VPBookmarkEditorLibrary::ActivateBookmarkInLevelEditorByIndex - Unable to get viewport client");
		return false;
	}

	UWorld* World = Client->GetWorld();
	if (World == nullptr)
	{
		UE_LOGF(LogVPBookmarkEditor, Warning, "VPBookmarkEditorLibrary::ActivateBookmarkInLevelEditorByIndex - Unable to get world");
		return false;
	}

	AWorldSettings* WorldSettings = World->GetWorldSettings();
	if (WorldSettings == nullptr)
	{
		UE_LOGF(LogVPBookmarkEditor, Warning, "VPBookmarkEditorLibrary::ActivateBookmarkInLevelEditorByIndex - Unable to get world settings");
		return false;
	}

	const TArray<UBookmarkBase*>& Bookmarks = WorldSettings->GetBookmarks();
	if (!Bookmarks.IsValidIndex(BookmarkIndex) || Bookmarks[BookmarkIndex] == nullptr)
	{
		UE_LOGF(LogVPBookmarkEditor, Warning, "VPBookmarkEditorLibrary::ActivateBookmarkInLevelEditorByIndex - Invalid bookmark index %d", BookmarkIndex);
		return false;
	}

	IBookmarkTypeTools::Get().JumpToBookmark(BookmarkIndex, nullptr, Client);
	return true;
}


AActor* UVPBookmarkEditorBlueprintLibrary::AddBookmarkAtCurrentLevelEditorPosition(const TSubclassOf<AActor> ActorClass, const FVPBookmarkCreationContext CreationContext, const FVector Offset, const bool bFlattenRotation)
{
	if (ActorClass.Get() == nullptr)
	{
		UE_LOGF(LogVPBookmarkEditor, Warning, "VPBookmarkEditorLibrary::AddBookmarkAtCurrentLevelEditorPosition - Invalid actor class");
		return nullptr;
	}

	FEditorViewportClient* Client = BookmarkLibrary::GetUsableViewportClient();
	if (Client == nullptr)
	{
		UE_LOGF(LogVPBookmarkEditor, Warning, "VPBookmarkEditorLibrary::AddBookmarkAtCurrentLevelEditorPosition - Unable to get viewport client");
		return nullptr;
	}

	UWorld* World = Client->GetWorld();
	if (World == nullptr)
	{
		UE_LOGF(LogVPBookmarkEditor, Warning, "VPBookmarkEditorLibrary::AddBookmarkAtCurrentLevelEditorPosition - Unable to get world");
		return nullptr;
	}

	AWorldSettings* WorldSettings = World->GetWorldSettings();
	if (WorldSettings == nullptr)
	{
		UE_LOGF(LogVPBookmarkEditor, Warning, "VPBookmarkEditorLibrary::AddBookmarkAtCurrentLevelEditorPosition - Unable to get world settings");
		return nullptr;
	}

	return FVPBookmarkTypeActions::SpawnBookmark(*Client, ActorClass, CreationContext, Offset, bFlattenRotation);
}


void UVPBookmarkEditorBlueprintLibrary::GetAllActorsClassThamImplementsVPBookmarkInterface(TArray<TSubclassOf<AActor>>& OutActorClasses)
{
	OutActorClasses.Reset();

	TArray<UClass*> Classes;
	GetDerivedClasses(AActor::StaticClass(), Classes);

	for (UClass* Class : Classes)
	{
		if (Class->ImplementsInterface(UVPBookmarkProvider::StaticClass()))
		{
			OutActorClasses.Add(Class);
		}
	}
}
