// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPBookmarkEditorModule.h"
#include "VPBookmarkTypeActions.h"
#include "Modules/ModuleManager.h"

#include "Bookmarks/IBookmarkTypeTools.h"


LLM_DEFINE_TAG(VirtualProduction_VPBookmarkEditor);
DEFINE_LOG_CATEGORY(LogVPBookmarkEditor);


void FVPBookmarkEditorModule::StartupModule()
{
	LLM_SCOPE_BYTAG(VirtualProduction_VPBookmarkEditor);

	BookmarkTypeActions = MakeShared<FVPBookmarkTypeActions>();
	IBookmarkTypeTools::Get().RegisterBookmarkTypeActions(BookmarkTypeActions.ToSharedRef());
}


void FVPBookmarkEditorModule::ShutdownModule()
{
	LLM_SCOPE_BYTAG(VirtualProduction_VPBookmarkEditor);

	IBookmarkTypeTools::Get().UnregisterBookmarkTypeActions(BookmarkTypeActions.ToSharedRef());
	BookmarkTypeActions.Reset();
}


IMPLEMENT_MODULE(FVPBookmarkEditorModule, VPBookmarkEditor)
