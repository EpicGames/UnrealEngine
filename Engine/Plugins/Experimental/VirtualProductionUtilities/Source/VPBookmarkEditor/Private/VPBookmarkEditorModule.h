// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

DECLARE_LOG_CATEGORY_EXTERN(LogVPBookmarkEditor, Log, Log);


class FVPBookmarkTypeActions;


class FVPBookmarkEditorModule : public IModuleInterface
{
public:
	TSharedPtr<FVPBookmarkTypeActions> BookmarkTypeActions;

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
