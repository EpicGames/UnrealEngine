// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPBookmarkTypeActions.h"
#include "IVPBookmarkProvider.h"
#include "VPBookmark.h"
#include "VPBookmarkContext.h"
#include "VPBookmarkEditorModule.h"

#include "Editor.h"
#include "EditorViewportClient.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"
#include "IVREditorModule.h"
#include "IXRTrackingSystem.h"
#include "LevelEditorViewport.h"
#include "ViewportWorldInteraction.h"
#include "VREditorMode.h"


namespace VPBookmark
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	static UViewportWorldInteraction* GetViewportWorldInteraction()
	{
		if (UEditorWorldExtensionManager* ExtensionManager = GEditor->GetEditorWorldExtensionsManager())
		{
			check(GEditor);
			if (UEditorWorldExtensionCollection* Collection = ExtensionManager->GetEditorWorldExtensions(GEditor->GetEditorWorldContext().World()))
			{
				return Cast<UViewportWorldInteraction>(Collection->FindExtension(UViewportWorldInteraction::StaticClass()));
			}
		}
		return nullptr;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS


	static FTransform GetEditorViewportTransform(FEditorViewportClient& InViewportClient)
	{
		FRotator ViewportRotation = FRotator::ZeroRotator;
		if (!InViewportClient.IsOrtho())
		{
			ViewportRotation = InViewportClient.GetViewRotation();
		}
		const FVector ViewportLocation = InViewportClient.GetViewLocation();

		return FTransform(ViewportRotation, ViewportLocation, FVector::OneVector);
	}
}


void FVPBookmarkTypeActions::ActivateBookmark(UVPBookmark* InBookmark, FEditorViewportClient& InViewportClient)
{
	if (AActor* BookmarkActorPtr = InBookmark->OwnedActor.Get())
	{
		FViewportCameraTransform& Transform = InViewportClient.GetViewTransform();
		const FTransform& ActorTransform = BookmarkActorPtr->GetTransform();
		const FRotator ActorRotation = ActorTransform.Rotator();
		const FVector ActorLocation = ActorTransform.GetLocation();

		// Set Location
		FVector Offset = FVector(0, 0, 0);
		

		// Set Rotation
		bool bIsRotationSet = false;
		FRotator UseRotation = ActorRotation;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (IVREditorModule::IsAvailable())
		{
			IVREditorModule& VREditorModule = IVREditorModule::Get();
			if (VREditorModule.IsVREditorModeActive())
			{
				if (UVREditorMode* EditorMode = VREditorModule.GetVRMode())
				{
					UseRotation.Pitch = 0.0f;
					UseRotation.Roll = 0.0f;
					Transform.SetRotation(UseRotation);
					bIsRotationSet = true;

					// Get HMD location in room space but scaled to the floor.
					FVector HMDLocationOffset = EditorMode->GetRoomSpaceHeadTransform().GetLocation() * FVector(1.f, 1.f, 0.f);
					Offset -= HMDLocationOffset;
				}
			}
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		Transform.SetLocation(ActorLocation - Offset);

		if (!bIsRotationSet)
		{
			if (InBookmark->CachedViewportData.bFlattenRotation)
			{
				UseRotation.Pitch = InBookmark->CachedViewportData.LookRotation.Pitch;
				UseRotation.Roll = InBookmark->CachedViewportData.LookRotation.Roll;
			}
			Transform.SetRotation(UseRotation);
		}

		Transform.SetOrthoZoom(InBookmark->CachedViewportData.OrthoZoom);

		LastActiveBookmark = InBookmark;
		InBookmark->SetActive(true);
		OnBookmarkActivated.Broadcast(InBookmark);
	}
}


void FVPBookmarkTypeActions::DeactivateBookmark(UVPBookmark* Bookmark, FEditorViewportClient& Client)
{
	if (Bookmark->IsActive())
	{
		LastActiveBookmark = nullptr;

		Bookmark->SetActive(false);
		OnBookmarkDeactivated.Broadcast(Bookmark);
	}
}


TSubclassOf<UBookmarkBase> FVPBookmarkTypeActions::GetBookmarkClass()
{
	return UVPBookmark::StaticClass();
}


void FVPBookmarkTypeActions::InitFromViewport(UBookmarkBase* InBookmark, FEditorViewportClient& InViewportClient)
{
	if (UVPBookmark* VPBookmark = Cast<UVPBookmark>(InBookmark))
	{
		if (AActor* BookmarkActorPtr = VPBookmark->OwnedActor.Get())
		{
			BookmarkActorPtr->Modify();

			FTransform HeadTransform = VPBookmark::GetEditorViewportTransform(InViewportClient);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			// TODO?: Do we want this logic transitioned to XR Creative?
			if (IVREditorModule::IsAvailable())
			{
				IVREditorModule& VREditorModule = IVREditorModule::Get();
				if (VREditorModule.IsVREditorModeActive())
				{
					if (UVREditorMode* EditorMode = VREditorModule.GetVRMode())
					{
						HeadTransform = EditorMode->GetHeadTransform();

						// Disregard head space location z when placing the object
						FVector RoomSpaceLocationOffset = EditorMode->GetRoomSpaceHeadTransform().GetLocation();
						FVector HeadLocation = HeadTransform.GetLocation();
						HeadLocation.Z -= RoomSpaceLocationOffset.Z;
						HeadTransform.SetLocation(HeadLocation);
					}
				}
			}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

			FRotator SpawnRotation = HeadTransform.Rotator();
			VPBookmark->CachedViewportData.LookRotation = SpawnRotation;

			// Disregard pitch and roll of where we're looking when placing the object
			if (VPBookmark->CachedViewportData.bFlattenRotation)
			{
				SpawnRotation.Pitch = 0;
				SpawnRotation.Roll = 0;
				HeadTransform.SetRotation(FQuat(SpawnRotation));
			}

			FVector SpawnLocation = HeadTransform.GetLocation() + HeadTransform.TransformVector(VPBookmark->CachedViewportData.JumpToOffsetLocation);
			BookmarkActorPtr->SetActorLocationAndRotation(SpawnLocation, SpawnRotation);


			const FViewportCameraTransform& Transform = InViewportClient.GetViewTransform();
			VPBookmark->CachedViewportData.OrthoZoom = Transform.GetOrthoZoom();
		}
		else
		{
			// If no actor was spawned with that bookmark, clear it.
			int32 BookmarkIndex = INDEX_NONE;
			if (const UWorld* World = InViewportClient.GetWorld())
			{
				if (AWorldSettings* WorldSettings = World->GetWorldSettings())
				{
					BookmarkIndex = WorldSettings->GetBookmarks().Find(VPBookmark);
					WorldSettings->ClearBookmark(BookmarkIndex);
					VPBookmark = nullptr;
				}
			}

			UE_LOGF(LogVPBookmarkEditor, Warning, "FVPBookmarkTypeActions::InitFromViewport has no valid component Bookmark: %ls Index: %d)", *GetPathNameSafe(VPBookmark), BookmarkIndex);
		}

		// Reactivate the bookmark with the new parameters.
		if (VPBookmark && VPBookmark->IsActive())
		{
			DeactivateBookmark(VPBookmark, InViewportClient);
			ActivateBookmark(VPBookmark, InViewportClient);
		}
	}
}


void FVPBookmarkTypeActions::JumpToBookmark(class UBookmarkBase* InBookmark, const TSharedPtr<struct FBookmarkBaseJumpToSettings> InSettings, class FEditorViewportClient& InViewportClient)
{
	if (UVPBookmark* Bookmark = LastActiveBookmark.Get())
	{
		DeactivateBookmark(Bookmark, InViewportClient);
	}

	if (UVPBookmark* Bookmark = Cast<UVPBookmark>(InBookmark))
	{
		ActivateBookmark(Bookmark, InViewportClient);
	}
}


AActor* FVPBookmarkTypeActions::SpawnBookmark(FEditorViewportClient& InViewportClient, const TSubclassOf<AActor> InActorClass, const FVPBookmarkCreationContext& InCreationContext, const FVector& InOffset, const bool bInFlattenRotation)
{
	if (InActorClass.Get() == nullptr)
	{
		UE_LOGF(LogVPBookmarkEditor, Error, "VPBookmarkTypeActions::SpawnBookmark - Invalid class");
		return nullptr;
	}

	if (!InActorClass.Get()->ImplementsInterface(UVPBookmarkProvider::StaticClass()))
	{
		UE_LOGF(LogVPBookmarkEditor, Warning, "VPBookmarkTypeActions::SpawnBookmark - The class '%ls' doesn't implement IVPBookmarkProvider", *(InActorClass.Get()->GetName()));
		return nullptr;
	}

	UWorld* World = InViewportClient.GetWorld();
	if (World == nullptr)
	{
		UE_LOGF(LogVPBookmarkEditor, Error, "VPBookmarkTypeActions::SpawnBookmark - Unable to get world");
		return nullptr;
	}

	AWorldSettings* WorldSettings = World->GetWorldSettings();
	if (WorldSettings == nullptr)
	{
		UE_LOGF(LogVPBookmarkEditor, Error, "VPBookmarkTypeActions::SpawnBookmark - Unable to get world settings");
		return nullptr;
	}

	// Create the Bookmark
	UVPBookmark* NewBookmark = Cast<UVPBookmark>(WorldSettings->AddBookmark(UVPBookmark::StaticClass(), true));
	if (NewBookmark == nullptr)
	{
		UE_LOGF(LogVPBookmarkEditor, Warning, "VPBookmarkTypeActions::SpawnBookmark - Unable to add bookmark");
		return nullptr;
	}
	NewBookmark->Modify();

	// Create the actor
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	AActor* SpawnedActor = World->SpawnActor<AActor>(InActorClass.Get(), SpawnInfo);
	if (SpawnedActor == nullptr)
	{
		UE_LOGF(LogVPBookmarkEditor, Error, "VPBookmarkTypeActions::SpawnBookmark - Unable to spawn the actor");
		WorldSettings->ClearBookmark(NewBookmark->GetBookmarkIndex());
		return nullptr;
	}
	SpawnedActor->Modify();

	// Initialize the actor
	SpawnedActor->SetFolderPath(TEXT("Bookmark"));

	// Initialize the bookmark
	NewBookmark->OwnedActor = SpawnedActor;
	NewBookmark->CachedViewportData.JumpToOffsetLocation = InOffset;
	NewBookmark->CachedViewportData.bFlattenRotation = bInFlattenRotation;
	NewBookmark->CreationContext = InCreationContext;
	if (NewBookmark->CreationContext.CategoryName.IsNone())
	{
		NewBookmark->CreationContext.CategoryName = InActorClass.Get()->GetFName();
	}

	FVPBookmarkEditorModule& LayersModule = FModuleManager::LoadModuleChecked<FVPBookmarkEditorModule>(TEXT("VPBookmarkEditor"));
	LayersModule.BookmarkTypeActions->InitFromViewport(NewBookmark, InViewportClient);

	if (SpawnedActor->GetClass()->ImplementsInterface(UVPBookmarkProvider::StaticClass()))
	{
		IVPBookmarkProvider::Execute_OnBookmarkChanged(SpawnedActor, NewBookmark);
	}

	return SpawnedActor;
}
