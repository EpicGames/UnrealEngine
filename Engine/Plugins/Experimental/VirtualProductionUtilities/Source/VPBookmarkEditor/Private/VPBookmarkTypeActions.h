// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Bookmarks/IBookmarkTypeActions.h"
#include "VPBookmark.h"

class AActor;
class FEditorViewportClient;
class UBookmarkBase;
struct FBookmarkBaseJumpToSettings;

DECLARE_MULTICAST_DELEGATE_OneParam(FVPBookmarkActivated, UVPBookmark*);
DECLARE_MULTICAST_DELEGATE_OneParam(FVPBookmarkDeactivated, UVPBookmark*);

class FVPBookmarkTypeActions : public IBookmarkTypeActions
{
private:
	TWeakObjectPtr<UVPBookmark> LastActiveBookmark;

public:
	FVPBookmarkActivated OnBookmarkActivated;
	FVPBookmarkDeactivated OnBookmarkDeactivated;

private:
	void ActivateBookmark(UVPBookmark* Bookmark, FEditorViewportClient& Client);
	void DeactivateBookmark(UVPBookmark* Bookmark, FEditorViewportClient& Client);

public:
	virtual TSubclassOf<UBookmarkBase> GetBookmarkClass() override;
	virtual void InitFromViewport(UBookmarkBase* InBookmark, FEditorViewportClient& InViewportClient) override;
	virtual void JumpToBookmark(UBookmarkBase* Bookmark, const TSharedPtr<FBookmarkBaseJumpToSettings> InSettings, FEditorViewportClient& InViewportClient) override;

	static AActor* SpawnBookmark(FEditorViewportClient& InViewportClient, const TSubclassOf<AActor> InActorClass, const FVPBookmarkCreationContext& InCreationContext, const FVector& InOffset, const bool bInFlattenRotation);
};