// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "VPBookmark.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"


class STableViewBase;
class SScrollBox;

namespace VPBookmarkList
{
	class SVPBookmarkCategoryListView;
}

/*
 * SVPBookmarkListView
 */
class VPBOOKMARKEDITOR_API SVPBookmarkListView : public SCompoundWidget
{
private:
	using Super = SCompoundWidget;
	friend VPBookmarkList::SVPBookmarkCategoryListView;

public:
	SLATE_BEGIN_ARGS(SVPBookmarkListView) { }
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	~SVPBookmarkListView();

private:
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void OnMapChanged(uint32 MapChangeFlags);
	void PopulateBookmarks();

	void OnBookmarkListModified(UVPBookmark* Bookmark);
	void OnBookmarkSelected(TSharedPtr<VPBookmarkList::SVPBookmarkCategoryListView> BookmarkCategory, TWeakObjectPtr<UVPBookmark> Selected, ESelectInfo::Type SelectionType);

	TWeakObjectPtr<UVPBookmark> GetSelectedBookmark();

private:
	bool bBookmarkListViewDirty;
	bool bInSelection;
	TArray<TSharedPtr<VPBookmarkList::SVPBookmarkCategoryListView>> BookmarkCategories;
	TSharedPtr<SScrollBox> BookmarkCategoryContainer;
	TWeakObjectPtr<UVPBookmark> SelectedBookmark;
};

