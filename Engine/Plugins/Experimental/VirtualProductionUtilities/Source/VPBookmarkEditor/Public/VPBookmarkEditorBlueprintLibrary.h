// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "VPBookmarkContext.h"
#include "VPBookmarkEditorBlueprintLibrary.generated.h"


class UVPBookmark;


UCLASS()
class VPBOOKMARKEDITOR_API UVPBookmarkEditorBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Bookmarks")
	static bool JumpToBookmarkInLevelEditor(const UVPBookmark* Bookmark);

	UFUNCTION(BlueprintCallable, Category = "Bookmarks")
	static bool JumpToBookmarkInLevelEditorByIndex(const int32 BookmarkIndex);

	UFUNCTION(BlueprintCallable, Category = "Bookmarks")
	static AActor* AddBookmarkAtCurrentLevelEditorPosition(const TSubclassOf<AActor> ActorClass, const FVPBookmarkCreationContext CreationContext, const FVector Offset, const bool bFlattenRotation = true);

	UFUNCTION(BlueprintCallable, Category = "Bookmarks")
	static void GetAllActorsClassThamImplementsVPBookmarkInterface(TArray<TSubclassOf<AActor>>& OutActorClasses);
};
