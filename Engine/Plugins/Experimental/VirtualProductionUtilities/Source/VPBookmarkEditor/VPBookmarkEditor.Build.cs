// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VPBookmarkEditor : ModuleRules
{
	public VPBookmarkEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"VPBookmark",
				"Core",
				"CoreUObject",
				"Engine",
				"SlateCore",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"EditorFramework",		
				"HeadMountedDisplay",
				"InputCore",
				"LevelEditor",
				"Slate",
				"ViewportInteraction",
				"VREditor",
				"UnrealEd",
			}
		);
	}
}
