// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actors/VPBookmarkActor.h"

#include "Libraries/VPBlueprintLibrary.h"
#include "VPBookmark.h"
#include "VPBookmarkBlueprintLibrary.h"
#include "VPBookmarkSettings.h"
#include "VPUtilitiesModule.h"

#include "UObject/ConstructorHelpers.h"
#include "Engine/StaticMesh.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SplineMeshComponent.h"
#include "Components/TextRenderComponent.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "CineCameraSceneCaptureComponent.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/Texture2D.h"
#include "Logging/MessageLog.h"

AVPBookmarkActor::AVPBookmarkActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bStartWithTickEnabled = true;

	//Root Component

	BookmarkMeshComponent = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("BookmarkMesh"));
	SetRootComponent(BookmarkMeshComponent);
	
	if (!IsRunningCommandlet())
	{
		static ConstructorHelpers::FObjectFinder<UStaticMesh> MeshFinder(*GetDefault<UVPBookmarkSettings>()->BookmarkMeshPath.ToString());
		if (MeshFinder.Succeeded())
		{
			BookmarkStaticMesh = MeshFinder.Object;
		}
		else
		{
			UE_LOGF(LogVPUtilities, Warning, "Failed to load bookmark spline: %ls", *GetDefault<UVPBookmarkSettings>()->BookmarkMeshPath.ToString());
		}

	}

	BookmarkMeshComponent->SetStaticMesh(BookmarkStaticMesh);
	
	if (!IsRunningCommandlet())
	{
		static ConstructorHelpers::FObjectFinder<UMaterialInterface> MaterialFinder(*GetDefault<UVPBookmarkSettings>()->BookmarkMaterialPath.ToString());

		if (MaterialFinder.Succeeded())
		{
			BookmarkMaterial = MaterialFinder.Object;
		}
		else
		{
			UE_LOGF(LogVPUtilities, Warning, "Failed to load bookmark material: %ls", *GetDefault<UVPBookmarkSettings>()->BookmarkMaterialPath.ToString());
		}
	}
	//Apply found material to all material slots on the BookmarkMeshComponent
	for (int32 i = 0; i < BookmarkMeshComponent->GetNumMaterials(); i++)
	{ 
		BookmarkMeshComponent->SetMaterial(i, BookmarkMaterial);
	}

	BookmarkColor = FLinearColor(0.817708f, 0.107659f, 0.230336f);

	//SplineMesh setup
	SplineMeshComponent = CreateDefaultSubobject<USplineMeshComponent>(TEXT("SplineMesh"));
	SplineMeshComponent->SetMobility(EComponentMobility::Movable);
	SplineMeshComponent->SetupAttachment(BookmarkMeshComponent);
	SplineMeshComponent->SetVisibility(false);
	
	if (!IsRunningCommandlet())
	{
		static ConstructorHelpers::FObjectFinder<UStaticMesh> SplineMeshFinder(*GetDefault<UVPBookmarkSettings>()->BookmarkSplineMeshPath.ToString());

		if (SplineMeshFinder.Succeeded())
		{
			SplineStaticMesh = SplineMeshFinder.Object;
		}
		else
		{
			UE_LOGF(LogVPUtilities, Warning, "Failed to load bookmark material: %ls", *GetDefault<UVPBookmarkSettings>()->BookmarkSplineMeshPath.ToString());
		}
	}

	SplineMeshComponent->SetStaticMesh(SplineStaticMesh);

	if (!IsRunningCommandlet())
	{
		static ConstructorHelpers::FObjectFinder<UMaterialInterface> SplineMaterialFinder(*GetDefault<UVPBookmarkSettings>()->BookmarkSplineMeshMaterialPath.ToString());

		if (SplineMaterialFinder.Succeeded())
		{
			SplineMaterialInstance = SplineMaterialFinder.Object;
		}
		else
		{
			UE_LOGF(LogVPUtilities, Warning, "Failed to load bookmark spline material: %ls", *GetDefault<UVPBookmarkSettings>()->BookmarkSplineMeshMaterialPath.ToString());
		}
	}
	
	SplineMeshComponent->SetMaterial(0, SplineMaterialInstance);

	//Text Render setup
	NameTextRenderComponent = CreateDefaultSubobject<UTextRenderComponent>(TEXT("NameTextRender"));
	NameTextRenderComponent->SetupAttachment(BookmarkMeshComponent);
	NameTextRenderComponent->SetWorldSize(36);
	NameTextRenderComponent->AddRelativeLocation(FVector(0.f, 0.f, 70.f));
	NameTextRenderComponent->HorizontalAlignment = EHorizTextAligment(EHTA_Center);
	
	if (!IsRunningCommandlet())
	{
		static ConstructorHelpers::FObjectFinder<UMaterialInterface> LabelMaterialFinder(*GetDefault<UVPBookmarkSettings>()->BookmarkLabelMaterialPath.ToString());

		if (LabelMaterialFinder.Succeeded())
		{
			LabelMaterialInstance = LabelMaterialFinder.Object;
		}
		else
		{
			UE_LOGF(LogVPUtilities, Warning, "Failed to load bookmark spline material: %ls", *GetDefault<UVPBookmarkSettings>()->BookmarkLabelMaterialPath.ToString());
		}
	}

	NameTextRenderComponent->SetMaterial(0, LabelMaterialInstance);

	CameraComponent = CreateDefaultSubobject<UCineCameraComponent>(TEXT("Camera"));
	CameraComponent->SetupAttachment(BookmarkMeshComponent);
	CameraComponent->SetVisibility(false, true);
	CameraComponent->SetVisibleFlag(false);
#if WITH_EDITOR
	CameraComponent->SetCameraMesh(nullptr);// set CameraMesh to null when we are in editor view
#endif

	SceneCaptureComponent = CreateDefaultSubobject<UCineCaptureComponent2D>(TEXT("SceneCapture"));
	SceneCaptureComponent->SetupAttachment(CameraComponent);
	SceneCaptureComponent->bCaptureEveryFrame = false;
	SceneCaptureComponent->bCaptureOnMovement = false;

}

void AVPBookmarkActor::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);

	NameTextRenderComponent->SetWorldRotation(FRotator(0.f, 0.f, 0.f));
	BookmarkRotation = GetActorRotation();
	BookmarkMeshComponent->SetWorldRotation(FRotator(0.f,BookmarkRotation.Yaw,0.f));

#if WITH_EDITOR
	if (GIsEditor)
	{
	FEditorScriptExecutionGuard ScriptGuard;
	EditorTick(DeltaSeconds);
	}
#endif
}

void AVPBookmarkActor::CaptureSnapshot()
{
#if WITH_EDITOR
	if (!SceneCaptureComponent->TextureTarget)
	{
		UE_LOGF(LogVPUtilities, Verbose, "Creating transient texture target");
		UTextureRenderTarget2D* NewRenderTarget2D = NewObject<UTextureRenderTarget2D>();
		NewRenderTarget2D->RenderTargetFormat = RTF_RGBA8_SRGB;
		
		int32 Width, Height;
		if (CameraComponent->AspectRatio >= 1.)
		{
			Width = SceneCaptureComponent->RenderTargetHighestDimension;
			Height = FMath::Max( Width / CameraComponent->AspectRatio, 1);
		}
		else
		{
			Height = SceneCaptureComponent->RenderTargetHighestDimension;
			Width = FMath::Max( Height * CameraComponent->AspectRatio, 1);
		}
		NewRenderTarget2D->InitAutoFormat(Width, Height);
		NewRenderTarget2D->UpdateResourceImmediate(true);

		SceneCaptureComponent->TextureTarget = NewRenderTarget2D;
	}
	if (!SnapshotTexture)
	{
		UE_LOGF(LogVPUtilities, Verbose, "Creating transient snapshot texture");
		SnapshotTexture = UTexture2D::CreateTransient(SceneCaptureComponent->TextureTarget->SizeX, SceneCaptureComponent->TextureTarget->SizeY, EPixelFormat::PF_B8G8R8A8);
	}

	if (SceneCaptureComponent->TextureTarget)
	{
		if (!SceneCaptureComponent->TextureTarget->GameThread_GetRenderTargetResource())
		{
			UE_LOGF(LogVPUtilities, Warning, "Unable to get render target resource");
			return;
		}
		SceneCaptureComponent->CaptureScene();
	}

	FText ErrorMessage;
	if (!SceneCaptureComponent->TextureTarget->UpdateTexture(SnapshotTexture, /*InFlags = */static_cast<EConstructTextureFlags>(CTF_Default | CTF_ForceOpaque), /*InAlphaOverride = */nullptr, /*InOnTextureChangingDelegate = */[](UTexture*) {}, &ErrorMessage))
	{
		FMessageLog("VPBookmarkActor").Warning(ErrorMessage);
		return;
	}

#endif
}

void AVPBookmarkActor::UpdateTimestamp()
{
	Timestamp = FDateTime::UtcNow();
}


//VP Interaction InterfaceEvents

void AVPBookmarkActor::OnBookmarkActivation_Implementation(UVPBookmark* BookmarkOut, bool bIsActive)
{
	UE_LOGF(LogVPUtilities, Display, "Bookmark Created");
}

void AVPBookmarkActor::OnBookmarkChanged_Implementation(UVPBookmark* BookmarkOut)
{
	AActor* BookmarkActor = BookmarkOut->GetAssociatedBookmarkActor();
	IVPBookmarkProvider* BookmarkInterface = Cast<IVPBookmarkProvider>(BookmarkActor);
	if (BookmarkInterface)
	{
		BookmarkInterface->Execute_GenerateBookmarkName(BookmarkActor);
	}
	BookmarkObject = BookmarkOut;
	UE_LOGF(LogVPUtilities, Display, "Bookmark Updated");
}

void AVPBookmarkActor::UpdateBookmarkSplineMeshIndicator_Implementation()
{
	UVPBlueprintLibrary::VPBookmarkSplineMeshIndicatorSetStartAndEnd(SplineMeshComponent);
}

void AVPBookmarkActor::HideBookmarkSplineMeshIndicator_Implementation()
{
	UVPBlueprintLibrary::VPBookmarkSplineMeshIndicatorDisable(SplineMeshComponent);
}

void AVPBookmarkActor::GenerateBookmarkName_Implementation()
{
	FString GeneratedNumber;
	FString GeneratedLetter;
	UVPBookmarkBlueprintLibrary::CreateVPBookmarkName(this, FString("Bookmark %n"), GeneratedNumber, GeneratedLetter);
		
	NameTextRenderComponent->SetText(FText::AsCultureInvariant(GeneratedNumber));
}

//VP Interaction Interface Events

void AVPBookmarkActor::OnActorDroppedFromCarry_Implementation()
{
	UE_LOGF(LogVPUtilities, Display, "Bookmark %ls dropped from carry by VR Interactor", *this->GetName());
}

void AVPBookmarkActor::OnActorSelectedForTransform_Implementation()
{
	UE_LOGF(LogVPUtilities, Display, "Bookmark %ls selected by VR Interactor", *this->GetName());
}

void AVPBookmarkActor::OnActorDroppedFromTransform_Implementation()
{
	UE_LOGF(LogVPUtilities, Display, "Bookmark %ls dropped from transform dragging by VR Interactor", *this->GetName());
}


void AVPBookmarkActor::UpdateBookmarkColor(FLinearColor Color)
{
	if (BookmarkMeshComponent && BookmarkMeshComponent->GetStaticMesh() != nullptr)
	{
		UMaterialInterface* Material = BookmarkMeshComponent->GetMaterial(0);

		if (BookmarkMaterial != nullptr)
		{
			if (Material && !Material->IsA<UMaterialInstanceDynamic>())
			{			
				DynamicMaterial = UMaterialInstanceDynamic::Create(Material, DynamicMaterial, TEXT("BookmarkMaterial"));

				DynamicMaterial->ClearParameterValues();
				DynamicMaterial->SetVectorParameterValue(TEXT("UserColor"), Color);

				for (int32 i = 0; i < BookmarkMeshComponent->GetNumMaterials(); i++)
				{
					BookmarkMeshComponent->SetMaterial(i, DynamicMaterial);
				}
			}
			else //If DMIs are already setup set the color value
			{
				DynamicMaterial = Cast<UMaterialInstanceDynamic>(Material);
				DynamicMaterial->SetVectorParameterValue(TEXT("UserColor"), Color);
			}
		}
	}
}
#if WITH_EDITOR
void AVPBookmarkActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(AVPBookmarkActor, BookmarkColor))
	{
		AVPBookmarkActor::UpdateBookmarkColor(BookmarkColor);
	}
}
#endif

void AVPBookmarkActor::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	AVPBookmarkActor::UpdateBookmarkColor(BookmarkColor);
}
