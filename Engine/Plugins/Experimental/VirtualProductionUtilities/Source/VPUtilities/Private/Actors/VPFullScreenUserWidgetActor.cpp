// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actors/VPFullScreenUserWidgetActor.h"

#include "ViewportWidgetOverlay.h"

#define LOCTEXT_NAMESPACE "VPFullScreenUserWidgetActor"

/////////////////////////////////////////////////////
// AFullScreenUserWidgetActor

AFullScreenUserWidgetActor::AFullScreenUserWidgetActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITOR
	, bEditorDisplayRequested(false)
#endif //WITH_EDITOR
{
	ScreenUserWidget = CreateDefaultSubobject<UViewportWidgetOverlay>(TEXT("ScreenUserWidget"));

	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bStartWithTickEnabled = true;
	bAllowTickBeforeBeginPlay = true;
	SetActorTickEnabled(true);
	SetHidden(false);
}

void AFullScreenUserWidgetActor::PostInitializeComponents()
{
	Super::PostInitializeComponents();

#if WITH_EDITOR
	bEditorDisplayRequested = true;
#endif //WITH_EDITOR
}

void AFullScreenUserWidgetActor::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	bEditorDisplayRequested = true;
#endif //WITH_EDITOR
}

void AFullScreenUserWidgetActor::PostActorCreated()
{
	Super::PostActorCreated();

#if WITH_EDITOR
	bEditorDisplayRequested = true;
#endif //WITH_EDITOR
}

void AFullScreenUserWidgetActor::Destroyed()
{
	if (ScreenUserWidget)
	{
		ScreenUserWidget->Hide();
	}
	Super::Destroyed();
}

void AFullScreenUserWidgetActor::BeginPlay()
{
	RequestGameDisplay();

	Super::BeginPlay();
}

void AFullScreenUserWidgetActor::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);

	if (ScreenUserWidget)
	{
		UWorld* ActorWorld = GetWorld();
		if (ActorWorld && (ActorWorld->WorldType == EWorldType::Game || ActorWorld->WorldType == EWorldType::PIE))
		{
			ScreenUserWidget->Hide();
		}
	}
}

void AFullScreenUserWidgetActor::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);

#if WITH_EDITOR
	if (bEditorDisplayRequested)
	{
		bEditorDisplayRequested = false;
		RequestEditorDisplay();
	}
#endif //WITH_EDITOR

	if (ScreenUserWidget)
	{
		ScreenUserWidget->Tick(DeltaSeconds);
	}
}

void AFullScreenUserWidgetActor::RequestEditorDisplay()
{
#if WITH_EDITOR
	UWorld* ActorWorld = GetWorld();
	if (ScreenUserWidget && ActorWorld && ActorWorld->WorldType == EWorldType::Editor)
	{
		ScreenUserWidget->Display(ActorWorld);
	}
#endif //WITH_EDITOR
}

void AFullScreenUserWidgetActor::RequestGameDisplay()
{
	UWorld* ActorWorld = GetWorld();
	if (ScreenUserWidget && ActorWorld && (ActorWorld->WorldType == EWorldType::Game || ActorWorld->WorldType == EWorldType::PIE))
	{
		ScreenUserWidget->Display(ActorWorld);
	}
}

UUserWidget* AFullScreenUserWidgetActor::GetUserWidget() const
{
	if (ScreenUserWidget)
	{
		return ScreenUserWidget->GetWidget();
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
