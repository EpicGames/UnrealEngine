// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actors/VPRootActor.h"

#include "CineCameraActor.h"
#include "Components/ArrowComponent.h"
#include "Components/BillboardComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/Texture2D.h"
#include "UObject/ConstructorHelpers.h"

#if WITH_EDITOR
#include "Engine/LevelStreaming.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "LevelUtils.h"
#include "Logging/MessageLog.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/UObjectToken.h"
#endif

#define LOCTEXT_NAMESPACE "VirtualProductionUtilities"


AVPRootActor::AVPRootActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bAreComponentsVisible(true)
#if WITH_EDITOR
	, bReentrantPostEditMove(false)
#endif
{
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("Root"));

	RealWorldSceneRepresentation = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("RealWorldRepresentation"));
	if (RealWorldSceneRepresentation)
	{
		RealWorldSceneRepresentation->bHiddenInGame = false;
		RealWorldSceneRepresentation->SetupAttachment(RootComponent);
	}

#if WITH_EDITOR
	struct FConstructorStatics
	{
		FName ID_Sprite;
		FText NAME_Sprite;
		FColor SceneBaseColor;
		FVector SceneBaseSize;
		FConstructorStatics()
			: ID_Sprite(TEXT("VPRootActor"))
			, NAME_Sprite(LOCTEXT("RootSpriteInfo", "VP Root Actor"))
			, SceneBaseColor(100, 255, 255, 255)
			, SceneBaseSize(600.0f, 600.0f, 400.0f)
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	if (SpriteComponent)
	{
		ConstructorHelpers::FObjectFinderOptional<UTexture2D> RootTextureObject = TEXT("/VirtualProductionUtilities/Icons/S_VPRootActor");

		SpriteComponent->Sprite = RootTextureObject.Get();
		SpriteComponent->SetRelativeScale3D_Direct(FVector(0.5f, 0.5f, 0.5f));
		SpriteComponent->bHiddenInGame = false;
		SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Sprite;
		SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Sprite;
		SpriteComponent->bIsScreenSizeScaled = true;
		SpriteComponent->SetupAttachment(RootComponent);
	}

	ArrowComponent = CreateEditorOnlyDefaultSubobject<UArrowComponent>(TEXT("Arrow"));
	if (ArrowComponent)
	{
		ArrowComponent->ArrowColor = FColor(150, 200, 255);
		ArrowComponent->bTreatAsASprite = true;
		ArrowComponent->bHiddenInGame = false;
		ArrowComponent->SpriteInfo.Category = ConstructorStatics.ID_Sprite;
		ArrowComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Sprite;
		ArrowComponent->bIsScreenSizeScaled = true;
		ArrowComponent->SetupAttachment(RootComponent);
	}
#endif
}


void AVPRootActor::BeginPlay()
{
	if (IsRunningGame())
	{
		bAreComponentsVisible = false;
	}
	SetComponentsVisibility(bAreComponentsVisible);

	Super::BeginPlay();
}


namespace VPRootActor
{
	template<class T>
	T* GetFirstActor_Implementation(const AVPRootActor* RootActor)
	{
		if (USceneComponent* SceneComponent = RootActor->GetRootComponent())
		{
			TArray<USceneComponent*> ChildComponents;
			SceneComponent->GetChildrenComponents(true, ChildComponents);
			for (USceneComponent* ChildComponent : ChildComponents)
			{
				if (AActor* ChildActor = ChildComponent->GetOwner())
				{
					if (T* CineCameraActor = Cast<T>(ChildActor))
					{
						return CineCameraActor;
					}
				}
			}
		}

		return nullptr;
	}
}


ACineCameraActor* AVPRootActor::GetCineCameraActor_Implementation() const
{
	if (CinematicCamera != nullptr)
	{
		return CinematicCamera;
	}

	return VPRootActor::GetFirstActor_Implementation<ACineCameraActor>(this);
}


#if WITH_EDITOR
void AVPRootActor::ToggleComponentsVisibility()
{
	Modify();
	bAreComponentsVisible = !bAreComponentsVisible;
	SetComponentsVisibility(bAreComponentsVisible);
}


void AVPRootActor::MoveLevelToRootActor()
{
	ULevel* OwningLevel = GetLevel();
	UWorld* OwningWorld = GetWorld();
	ULevelStreaming* OwningStreamingLevel = nullptr;
	if (OwningWorld && OwningLevel)
	{
		ULevelStreaming *const * FoundItem = OwningWorld->GetStreamingLevels().FindByPredicate([OwningLevel](ULevelStreaming* InItem) { return InItem && InItem->GetLoadedLevel() == OwningLevel; });
		if (FoundItem)
		{
			OwningStreamingLevel = *FoundItem;
		}
	}

	if (OwningStreamingLevel)
	{
		FVector ActorLocation = GetActorLocation();
		FRotator ActorRotation = GetActorRotation();
		ActorRotation.Pitch = 0;
		ActorRotation.Roll = 0;
		FLevelUtils::SetEditorTransform(OwningStreamingLevel, FTransform(ActorRotation, ActorLocation));
		SetActorLocationAndRotation(ActorLocation, ActorRotation, false);
	}
}


void AVPRootActor::CheckForErrors()
{
	Super::CheckForErrors();

	if (GetCineCameraActor() == nullptr)
	{
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_CinematicCameraNull", "The Root Actor has doesn't have a Cinematic Camera Actor set.")));
	}
}


void AVPRootActor::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	if (bMoveLevelWithActor && !bReentrantPostEditMove)
	{
		if (PropertyChangedEvent.Property->GetFName() == USceneComponent::GetRelativeLocationPropertyName()
			|| PropertyChangedEvent.Property->GetFName() == USceneComponent::GetRelativeRotationPropertyName())
		{
			TGuardValue<bool> Tmp(bReentrantPostEditMove, true);
			MoveLevelToRootActor();
		}
	}
}


void AVPRootActor::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);

	if (bMoveLevelWithActor && !bReentrantPostEditMove && bFinished)
	{
		TGuardValue<bool> Tmp(bReentrantPostEditMove, true);
		MoveLevelToRootActor();
	}
}
#endif //WITH_EDITOR


void AVPRootActor::SetComponentsVisibility(bool bVisible)
{
	if (RealWorldSceneRepresentation)
	{
		RealWorldSceneRepresentation->SetVisibility(bVisible, true);
	}

#if WITH_EDITOR
	if (SpriteComponent)
	{
		SpriteComponent->SetVisibility(bVisible, true);
	}
	if (ArrowComponent)
	{
		ArrowComponent->SetVisibility(bVisible, true);
	}
#endif
}


#undef LOCTEXT_NAMESPACE
