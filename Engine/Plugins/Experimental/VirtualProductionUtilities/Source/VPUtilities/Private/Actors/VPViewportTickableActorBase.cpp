// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actors/VPViewportTickableActorBase.h"

#include "Engine/World.h"

AVPViewportTickableActorBase::AVPViewportTickableActorBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ViewportTickType(EVPViewportTickableFlags::Game | EVPViewportTickableFlags::Editor)
{
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bStartWithTickEnabled = true;
	SetActorTickEnabled(true);
	//we don't want virtual production objects to be visible by cameras
	SetActorHiddenInGame(true);
}


bool AVPViewportTickableActorBase::ShouldTickIfViewportsOnly() const
{
	if (UWorld* World = GetWorld())
	{
		switch (World->WorldType)
		{
		case EWorldType::Game:
		case EWorldType::GameRPC:
		case EWorldType::PIE:
			return static_cast<uint8>(ViewportTickType & EVPViewportTickableFlags::Game) != 0;
		case EWorldType::Editor:
			return static_cast<uint8>(ViewportTickType & EVPViewportTickableFlags::Editor) != 0;
		case EWorldType::EditorPreview:
			return static_cast<uint8>(ViewportTickType & EVPViewportTickableFlags::EditorPreview) != 0;
		case EWorldType::GamePreview:
			return static_cast<uint8>(ViewportTickType & EVPViewportTickableFlags::GamePreview) != 0;
		}
	}
	return false;
}


void AVPViewportTickableActorBase::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);

#if WITH_EDITOR
	if (GIsEditor)
	{
		FEditorScriptExecutionGuard ScriptGuard;
		EditorTick(DeltaSeconds);
	}
#endif
}


void AVPViewportTickableActorBase::Destroyed()
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		FEditorScriptExecutionGuard ScriptGuard;
		EditorDestroyed();
	}
#endif

	Super::Destroyed();
}

void AVPViewportTickableActorBase::EditorLockLocation(bool bSetLockLocation)
{
#if WITH_EDITOR
	bLockLocation = bSetLockLocation;
#endif
}

void AVPViewportTickableActorBase::EditorTick_Implementation(float DeltaSeconds)
{

}

void AVPViewportTickableActorBase::EditorDestroyed_Implementation()
{

}
