// Copyright Epic Games, Inc. All Rights Reserved.

#include "Libraries/VPBlueprintLibrary.h"

#include "VPSettings.h"
#include "VPUtilitiesModule.h"
#include "Actors/VPBookmarkActor.h"

#include "Engine/World.h"
#include "HAL/IConsoleManager.h"
#include "Components/SplineMeshComponent.h"
#include "Misc/ComparisonUtility.h"

#if WITH_EDITOR
#include "Editor.h"
#include "IVREditorModule.h"
#include "LevelEditorViewport.h"
#include "Subsystems/UnrealEditorSubsystem.h"
#include "ViewportWorldInteraction.h"
#include "VPBookmarkEditorBlueprintLibrary.h"
#include "VREditorInteractor.h"
#endif


namespace VPBlueprintLibrary
{
#if WITH_EDITOR
	FLevelEditorViewportClient* GetViewPortClient()
	{
		return	GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient :
			GLastKeyLevelEditingViewportClient ? GLastKeyLevelEditingViewportClient :
			nullptr;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UViewportWorldInteraction* GetViewportWorldInteraction(FString& FailMessage)
	{
		if (FLevelEditorViewportClient* Client = GetViewPortClient())
		{
			UViewportWorldInteraction* ViewportWorldInteraction = nullptr;

			if (UEditorWorldExtensionManager* ExtensionManager = GEditor->GetEditorWorldExtensionsManager())
			{
				check(GEditor);
				if (UEditorWorldExtensionCollection* Collection = ExtensionManager->GetEditorWorldExtensions(GEditor->GetEditorWorldContext().World()))
				{
					ViewportWorldInteraction = Cast<UViewportWorldInteraction>(Collection->FindExtension(UViewportWorldInteraction::StaticClass()));

					if (ViewportWorldInteraction != nullptr)
					{
						return ViewportWorldInteraction;
					}
				}
			}
		}

		UE_LOGF(LogVPUtilities, Warning, "UVPBlueprintLibrary::GetViewportWorldInteraction - Failed to get VPI. %ls", *FailMessage);
		return nullptr;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

void UVPBlueprintLibrary::Refresh3DEditorViewport()
{
#if WITH_EDITOR	
	if (FLevelEditorViewportClient* VP = VPBlueprintLibrary::GetViewPortClient())
	{
		VP->Invalidate(true);
	}
#endif
}

AVPViewportTickableActorBase* UVPBlueprintLibrary::SpawnVPTickableActor(UObject* ContextObject, const TSubclassOf<AVPViewportTickableActorBase> ActorClass, const FVector Location, const FRotator Rotation)
{
	if (ActorClass.Get() == nullptr)
	{
		UE_LOGF(LogVPUtilities, Warning, "UVPBlueprintLibrary::SpawnVPTickableActor - The ActorClass is invalid");
		return nullptr;
	}

	UWorld* World = ContextObject ? ContextObject->GetWorld() : nullptr;
	if (World == nullptr)
	{
		UE_LOGF(LogVPUtilities, Warning, "UVPBlueprintLibrary::SpawnVPTickableActor - The ContextObject is invalid.");
		return nullptr;
	}

	FActorSpawnParameters SpawnParams;
	SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	AVPViewportTickableActorBase* NewActor = World->SpawnActor<AVPViewportTickableActorBase>(ActorClass.Get(), Location, Rotation, SpawnParams);
	return NewActor;
}


AActor* UVPBlueprintLibrary::SpawnBookmarkAtCurrentLevelEditorPosition(const TSubclassOf<AActor> ActorClass, const FVPBookmarkCreationContext CreationContext, const FVector Offset, const bool bFlattenRotation)
{
	AActor* Result = nullptr;
#if WITH_EDITOR
	Result = UVPBookmarkEditorBlueprintLibrary::AddBookmarkAtCurrentLevelEditorPosition(ActorClass, CreationContext, Offset, bFlattenRotation);
#endif
	return Result;
}

bool UVPBlueprintLibrary::JumpToBookmarkInLevelEditor(const UVPBookmark* Bookmark)
{
	bool bResult = false;
#if WITH_EDITOR
	bResult = UVPBookmarkEditorBlueprintLibrary::JumpToBookmarkInLevelEditor(Bookmark);
#endif
	return bResult;
}


FGameplayTagContainer UVPBlueprintLibrary::GetVirtualProductionRole()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// We can't depend upon the VP Roles subsystem because it will introduce a circular dependency so we have to
	// depend upon the deprecated method.
	return GetDefault<UVPSettings>()->GetRoles();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


FTransform UVPBlueprintLibrary::GetEditorViewportTransform()
{
#if WITH_EDITOR
	if (FLevelEditorViewportClient* Client = VPBlueprintLibrary::GetViewPortClient())
	{ 
		FRotator ViewportRotation(0, 0, 0);
		FVector ViewportLocation(0, 0, 0);
		
		if (!Client->IsOrtho())
		{
			ViewportRotation = Client->GetViewRotation();
		}

		ViewportLocation = Client->GetViewLocation();
		return FTransform(ViewportRotation, ViewportLocation, FVector::OneVector);
	}
#endif

	return FTransform();
}


FTransform UVPBlueprintLibrary::GetEditorVRHeadTransform()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if WITH_EDITOR
	FString ErrorText(TEXT("Head Transform will be invalid."));
	if (const UViewportWorldInteraction* VPI = VPBlueprintLibrary::GetViewportWorldInteraction(ErrorText))
	{
		return VPI->GetHeadTransform();
	}
#endif
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return FTransform::Identity;
}


FTransform UVPBlueprintLibrary::GetEditorVRRoomTransform()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if WITH_EDITOR
	FString ErrorText(TEXT("Room Transform will be invalid."));
	if (const UViewportWorldInteraction* VPI = VPBlueprintLibrary::GetViewportWorldInteraction(ErrorText))
	{
		return VPI->GetRoomTransform();
	}
#endif
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return FTransform::Identity;
}


void UVPBlueprintLibrary::SetGrabSpeed(const float Speed)
{
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("VI.DragScale"));
	CVar->Set(Speed);
}


bool UVPBlueprintLibrary::IsVREditorModeActive()
{
#if WITH_EDITOR
	if (IVREditorModule::IsAvailable())
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return IVREditorModule::Get().IsVREditorModeActive();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif

	return false;
}


FVector UVPBlueprintLibrary::GetVREditorLaserHoverLocation()
{
#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FString ErrorText(TEXT("VR laser hit location will be invalid."));
	if (const UViewportWorldInteraction* VPI = VPBlueprintLibrary::GetViewportWorldInteraction(ErrorText))
	{
		const TArray<UViewportInteractor*> Interactors = VPI->GetInteractors();

		for (UViewportInteractor* Interactor : Interactors)
		{
			if (UVREditorInteractor* EdInteractor = Cast<UVREditorInteractor>(Interactor))
			{
				if (EdInteractor->GetControllerType() == EControllerType::Laser)
				{
					//FVector Temp = EdInteractor->GetHoverLocation();
					//UE_LOGF(LogVPUtilities, Warning, "%ls", *Temp.ToString());

					return EdInteractor->GetInteractorData().LastHoverLocationOverUI;
				}
			}
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	return FVector(0);
}

FVector UVPBlueprintLibrary::GetLeftInteractorLocation()
{
#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FString ErrorText(TEXT("VR interactor location will be invalid."));
	if (const UViewportWorldInteraction* VPI = VPBlueprintLibrary::GetViewportWorldInteraction(ErrorText))
	{
		const TArray<UViewportInteractor*> Interactors = VPI->GetInteractors();

		for (UViewportInteractor* Interactor : Interactors)
		{
			if (UVREditorInteractor* EdInteractor = Cast<UVREditorInteractor>(Interactor))
			{
				if (EdInteractor->GetControllerType() == EControllerType::Laser)
				{
					const FVector InteractorLocation = EdInteractor->GetInteractorData().Transform.GetLocation();
					UE_LOGF(LogVPUtilities, Verbose , "%ls", *InteractorLocation.ToString());

					return InteractorLocation;
				}
			}
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	return FVector(0);
}

bool UVPBlueprintLibrary::EditorUndo()
{
#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FString ErrorText(TEXT("Undo did not execute."));
	if (UViewportWorldInteraction* VPI = VPBlueprintLibrary::GetViewportWorldInteraction(ErrorText))
	{
		VPI->Undo();
		return true;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	return false;
}


bool UVPBlueprintLibrary::EditorRedo()
{
#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FString ErrorText(TEXT("Redo did not execute."));
	if (UViewportWorldInteraction* VPI = VPBlueprintLibrary::GetViewportWorldInteraction(ErrorText))
	{
		VPI->Redo();
		return true;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	return false;
}


bool UVPBlueprintLibrary::EditorDeleteSelectedObjects()
{
#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FString ErrorText(TEXT("Delete did not execute."));
	if (UViewportWorldInteraction* VPI = VPBlueprintLibrary::GetViewportWorldInteraction(ErrorText))
	{
		VPI->DeleteSelectedObjects();
		return true;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	return false;
}

bool UVPBlueprintLibrary::EditorDuplicate()
{
#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FString ErrorText(TEXT("Duplicate did not execute."));
	if (UViewportWorldInteraction* VPI = VPBlueprintLibrary::GetViewportWorldInteraction(ErrorText))
	{
		VPI->Duplicate();
		return true;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	return false;
}

UWorld* UVPBlueprintLibrary::GetEditorWorld()
{
	UWorld* World = nullptr;
#if WITH_EDITOR
	if (UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>())
	{
		World = UnrealEditorSubsystem->GetEditorWorld();
	}

#endif
	return World;
}

void UVPBlueprintLibrary::VPBookmarkSplineMeshIndicatorSetStartAndEnd(USplineMeshComponent* SplineMesh)
{
	SplineMesh->SetVisibility(true);
	const FTransform SplineTransform = SplineMesh->GetComponentTransform();
		
	// @todo: Fix - GetVREditorLaserHoverLocation() does not return the correct hover location
	// USplineMeshComponent::SetEndPosition expects local space
PRAGMA_DISABLE_DEPRECATION_WARNINGS // TODO: XR Creative Framework relevant?
	SplineMesh->SetEndPosition(SplineTransform.InverseTransformPosition(IsVREditorModeActive() ? GetLeftInteractorLocation(): GetEditorViewportTransform().TransformPosition(FVector(80, 0, -20))));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UVPBlueprintLibrary::VPBookmarkSplineMeshIndicatorDisable(USplineMeshComponent* SplineMesh)
{
	SplineMesh->SetVisibility(false);
}

void UVPBlueprintLibrary::SortActorsByName(UPARAM(Ref) TArray<AActor*>& Actors, bool Ascending)
{
	Algo::Sort(Actors, [](const AActor* A, const AActor* B)
		{
			return UE::ComparisonUtility::CompareNaturalOrder(A->GetActorNameOrLabel(), B->GetActorNameOrLabel()) < 0;
		});

	if (!Ascending)
	{
		Algo::Reverse(Actors);
	}
}

void UVPBlueprintLibrary::SortVPBookmarkActorsByTimestamp(UPARAM(Ref) TArray<AVPBookmarkActor*>& Actors, bool Ascending)
{
	Algo::Sort(Actors, [](const AVPBookmarkActor* A, const AVPBookmarkActor* B)
		{
			return A->Timestamp < B->Timestamp;
		});

	if (!Ascending)
	{
		Algo::Reverse(Actors);
	}
}