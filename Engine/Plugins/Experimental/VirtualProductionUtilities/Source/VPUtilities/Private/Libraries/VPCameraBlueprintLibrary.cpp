// Copyright Epic Games, Inc. All Rights Reserved.

#include "Libraries/VPCameraBlueprintLibrary.h"

#include "VPUtilitiesModule.h"

#include "CameraRig_Rail.h"
#include "Components/SplineComponent.h"
#include "Engine/World.h"
#include "Kismet/GameplayStatics.h"


#if WITH_EDITOR
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Selection.h"
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FVPCameraRigSpawnParams::FVPCameraRigSpawnParams()
	: bUseWorldSpace(true)
	, bUseFirstPointAsSpawnLocation(false)
	, LinearApproximationMode(EVPCameraRigSpawnLinearApproximationMode::None)
	, LinearApproximationParam(1.f)
{}


ACameraRig_Rail* UVPCameraBlueprintLibrary::SpawnDollyTrackFromPoints(UObject* WorldContextObject, const TArray<FTransform>& Points, ESplinePointType::Type InterpType)
{
	UWorld* World = WorldContextObject ? WorldContextObject->GetWorld() : nullptr;
	if (World == nullptr)
	{
		UE_LOGF(LogVPUtilities, Warning, "UVPCameraBlueprintLibrary::SpawnDollyTrackFromPoints - Unable to get world");
		return nullptr;
	}

	if (Points.Num() <= 1)
	{
		UE_LOGF(LogVPUtilities, Warning, "UVPCameraBlueprintLibrary::SpawnDollyTrackFromPoints - Too few points");
		return nullptr;
	}


	const FTransform& Origin = Points[0];
	const FTransform OriginInverse = Origin.Inverse();
	const int32 NumPoints = Points.Num();
	const float Base = static_cast<float>(NumPoints - 1);

	ACameraRig_Rail* DollyTrack = World->SpawnActorDeferred<ACameraRig_Rail>(ACameraRig_Rail::StaticClass(), Origin, nullptr, nullptr, ESpawnActorCollisionHandlingMethod::AlwaysSpawn);

	USplineComponent* SplineComponent = DollyTrack->GetRailSplineComponent();
	SplineComponent->ClearSplinePoints(false);

	const FVector ArriveTangent = FVector::ZeroVector;
	const FVector LeaveTangent = FVector::ZeroVector;
	for (int32 i = 0; i < NumPoints; ++i)
	{
		FTransform LocalTransform = OriginInverse * Points[i];

		FSplinePoint NewPoint(
			static_cast<float>(i) / Base,
			LocalTransform.GetLocation(),
			ArriveTangent,
			LeaveTangent,
			LocalTransform.GetRotation().Rotator(),
			LocalTransform.GetScale3D(),
			InterpType
		);

		const bool bUpdateSpline = false;
		SplineComponent->AddPoint(NewPoint, bUpdateSpline);
	}

	SplineComponent->UpdateSpline();

	// Note: We previously called AutoSetTangents on the reparameterization table here.
	// With the recent refactor deprecating SplineCurves, there was no way to replace this code.
	// It did have the effect of setting tangents, but these tangents were not actually
	// used because the reparam points were all CIM_Linear, so evaluation did not require tangents.

	return Cast<ACameraRig_Rail>(UGameplayStatics::FinishSpawningActor(DollyTrack, Points[0]));
}


ACameraRig_Rail* UVPCameraBlueprintLibrary::SpawnDollyTrackFromPointsSmooth(UObject* WorldContextObject, const TArray<FTransform>& Points, ESplinePointType::Type InterpType)
{
	ACameraRig_Rail* DollyTrack = SpawnDollyTrackFromPoints(WorldContextObject, Points, InterpType);

	if (DollyTrack != nullptr)
	{
		USplineComponent* SplineComponent = DollyTrack->GetRailSplineComponent();

		const int32 Substeps = SplineComponent->ReparamStepsPerSegment;
		const int32 NumPoints = Points.Num();
		const float Base = static_cast<float>(NumPoints * Substeps - 1);

		SplineComponent->ReparamStepsPerSegment = 1;
		
		
		// Generate new points that are equally spaced.
		TArray<FSplinePoint> SplinePoints;
		SplinePoints.Reserve(Points.Num());

		int32 CurPoint = 0;
		for (int32 i = 0; i < NumPoints; ++i)
		{
			for (int32 j = 0; j < Substeps; ++j, ++CurPoint)
			{
				const float Step = static_cast<float>(CurPoint);
				const float DistanceAlongPoint = (Step) / Base;
				FTransform LocalTransform = SplineComponent->GetTransformAtDistanceAlongSpline(DistanceAlongPoint, ESplineCoordinateSpace::Local, /* bUseScale= */true);

				new (SplinePoints) FSplinePoint(
					Step / Base,
					LocalTransform.GetLocation(),
					/* InArriveTangent= */ FVector::ZeroVector,
					/* InLeaveTangent= */ FVector::ZeroVector,
					LocalTransform.GetRotation().Rotator(),
					LocalTransform.GetScale3D(),
					InterpType
				);
			}
		}

		SplineComponent->ClearSplinePoints(false);
		SplineComponent->AddPoints(SplinePoints, /* bUpdateSpline= */ true);
		
		// Note: We previously called AutoSetTangents on the reparameterization table here.
		// With the recent refactor deprecating SplineCurves, there was no way to replace this code.
		// It did have the effect of setting tangents, but these tangents were not actually
		// used because the reparam points were all CIM_Linear, so evaluation did not require tangents.
	}

	return DollyTrack;
}


ACameraRig_Rail* UVPCameraBlueprintLibrary::SpawnCameraRigFromPoints(UObject* WorldContextObject, const FTransform& RigTransform, const TArray<FVector>& Points, const FVPCameraRigSpawnParams& Params)
{
	ACameraRig_Rail* CameraRig = nullptr;

	UWorld* World = WorldContextObject ? WorldContextObject->GetWorld() : nullptr;
	if (World == nullptr)
	{
		UE_LOGF(LogVPUtilities, Warning, "UVPCameraBlueprintLibrary::SpawnCameraRigFromPoints - Unable to get world");
		return CameraRig;
	}

	if (Points.Num() <= 1)
	{
		UE_LOGF(LogVPUtilities, Warning, "UVPCameraBlueprintLibrary::SpawnCameraRigFromPoints - Too few points");
		return CameraRig;
	}

	FActorSpawnParameters SpawnParams;
	SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;

	FTransform UseTransform = Params.bUseFirstPointAsSpawnLocation ? FTransform(Points[0]) : RigTransform;
	ACameraRig_Rail* LocalCameraRig = World->SpawnActor<ACameraRig_Rail>(ACameraRig_Rail::StaticClass(), UseTransform, SpawnParams);

	if (LocalCameraRig == nullptr)
	{
		UE_LOGF(LogVPUtilities, Warning, "UVPCameraBlueprintLibrary::SpawnCameraRigFromPoints - Failed to spawn camera rig");
		return CameraRig;
	}

	CameraRig = LocalCameraRig;

	USplineComponent* SplineComponent = CameraRig->GetRailSplineComponent();
	if (SplineComponent == nullptr)
	{
		UE_LOGF(LogVPUtilities, Warning, "UVPCameraBlueprintLibrary::SpawnCameraRigFromPoints - Failed to get Spline");
		return CameraRig;
	}

	const ESplineCoordinateSpace::Type CoordinateSpace = Params.bUseWorldSpace ? ESplineCoordinateSpace::World : ESplineCoordinateSpace::Local;
	SplineComponent->SetSplinePoints(Points, CoordinateSpace);

	if (Params.LinearApproximationMode != EVPCameraRigSpawnLinearApproximationMode::None)
	{
		const FSplineCurves CachedLegacySplineData = SplineComponent->GetSplineCurves();
		TArray<FSplinePositionLinearApproximation> OutPoints;
		float Density = Params.LinearApproximationParam;

		if (Params.LinearApproximationMode == EVPCameraRigSpawnLinearApproximationMode::IntegrationStep)
		{
			// Convert integration step to density.
			Density = FMath::CeilToFloat(1.f / (CachedLegacySplineData.GetSplineLength() / FMath::Max(KINDA_SMALL_NUMBER, Density)));
		}

		FSplinePositionLinearApproximation::Build(CachedLegacySplineData, OutPoints, Density);

		TArray<FVector> NewPoints;
		NewPoints.Reserve(OutPoints.Num());
		for (const FSplinePositionLinearApproximation& ApproxPoint : OutPoints)
		{
			NewPoints.Add(ApproxPoint.Position);
		}

		// SplineComponent implicitly converts to LocalSpace, so since we've already built once this
		// should be safe.
		SplineComponent->SetSplinePoints(NewPoints, ESplineCoordinateSpace::Local);
	}

	return CameraRig;
}


ACameraRig_Rail* UVPCameraBlueprintLibrary::SpawnCameraRigFromActors(UObject* WorldContextObject, const FTransform& RigTransform, const TArray<AActor*>& Actors, const FVPCameraRigSpawnParams& Params)
{
	TArray<FVector> Points;
	Points.Reserve(Actors.Num());

	for (const AActor* Actor : Actors)
	{
		Points.Emplace(Actor->GetTransform().GetLocation());
	}

	return SpawnCameraRigFromPoints(WorldContextObject, RigTransform, Points, Params);
}


ACameraRig_Rail* UVPCameraBlueprintLibrary::SpawnCameraRigFromSelectedActors(UObject* WorldContextObject, const FTransform& RigTransform, const FVPCameraRigSpawnParams& Params)
{
#if WITH_EDITOR
	if (GEditor != nullptr)
	{
		TArray<AActor*> SelectedActors;
		SelectedActors.Reserve(GEditor->GetSelectedActorCount());

		for (auto It = GEditor->GetSelectedActorIterator(); It; ++It)
		{
			if (AActor* Actor = Cast<AActor>(*It))
			{
				SelectedActors.Emplace(Actor);
			}
		}

		return SpawnCameraRigFromActors(WorldContextObject, RigTransform, SelectedActors, Params);
	}
#endif

	UE_LOGF(LogVPUtilities, Warning, "UVPCameraBlueprintLibrary::SpawnCameraRigFromSelectedActors - Only callable from editor");
	return nullptr;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS