// Copyright Epic Games, Inc. All Rights Reserved.

#include "Libraries/VPRenderingBlueprintLibrary.h"

#include "Engine/GameEngine.h"
#include "SceneViewExtension.h"
#include "Slate/SceneViewport.h"

#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#include "ILevelEditor.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "SLevelViewport.h"
#endif // WITH_EDITOR


void UVPRenderingBlueprintLibrary::GenerateSceneViewExtensionIsActiveFunctorForViewportType(
	FSceneViewExtensionIsActiveFunctor& OutIsActiveFunction,
	bool bPIE,
	bool bSIE,
	bool bEditorActive,
	bool bGamePrimary)
{
	OutIsActiveFunction.IsActiveFunction = [=](
		const ISceneViewExtension* SceneViewExtension,
		const FSceneViewExtensionContext& Context)
	{
		if (!Context.Viewport)
		{
			return TOptional<bool>();
		}

#if WITH_EDITOR
		if (GIsEditor && GEditor)
		{
			// Activate the SVE if it is a PIE viewport.
			if (bPIE)
			{
				if (Context.Viewport->IsPlayInEditorViewport())
				{
					return TOptional<bool>(true);
				}
			}

			// Activate the SVE if it is the active Editor viewport.
			if (bEditorActive)
			{
				if (GEditor->GetActiveViewport() == Context.Viewport)
				{
					return TOptional<bool>(true);
				}
			}

			// Activate the SVE if it is a SIE viewport
			if (bSIE && GEditor->bIsSimulatingInEditor)
			{
				if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor")))
				{
					TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule->GetLevelEditorInstance().Pin();

					if (LevelEditor.IsValid())
					{
						for (TSharedPtr<SLevelViewport>& AssetViewport : LevelEditor->GetViewports())
						{
							if (AssetViewport->HasPlayInEditorViewport() && (AssetViewport->GetActiveViewport() == Context.Viewport))
							{
								return TOptional<bool>(true);
							}
						}
					}
				}
			}
		}
#endif // WITH_EDITOR

		// Activate the SVE if it is the game's primary viewport.
		if (bGamePrimary)
		{
			if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
			{
				if (GameEngine->SceneViewport.IsValid() && (GameEngine->SceneViewport.Get() == Context.Viewport))
				{
					return TOptional<bool>(true);
				}
			}
		}

		// If our viewport did not meet any of the criteria to activate the SVE, emit no opinion.
		return TOptional<bool>();
	};
}
