// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPGameMode.h"

#include "Actors/VPRootActor.h"

#include "EngineUtils.h"
#include "GameFramework/Actor.h"
#include "GameFramework/WorldSettings.h"

AActor* AVPGameMode::ChoosePlayerStart_Implementation(AController* Player)
{
	RootActor = nullptr;

	AWorldSettings* WorldSetting = GetWorld() ? GetWorld()->GetWorldSettings() : nullptr;
	if (WorldSetting)
	{
		UVPWorldAssetUserData* UserData = CastChecked<UVPWorldAssetUserData>(WorldSetting->GetAssetUserDataOfClass(UVPWorldAssetUserData::StaticClass()), ECastCheckedType::NullAllowed);
		if (UserData)
		{
			RootActor = UserData->LastSelectedRootActor.Get();
		}
	}

	if (RootActor == nullptr)
	{
		if (TActorIterator<AVPRootActor> It(GetWorld()); It)
		{
			RootActor = *It;
		}
	}

	AActor* Result = RootActor;
	if (Result == nullptr)
	{
		Result = Super::ChoosePlayerStart_Implementation(Player);
	}

	return Result;
}
