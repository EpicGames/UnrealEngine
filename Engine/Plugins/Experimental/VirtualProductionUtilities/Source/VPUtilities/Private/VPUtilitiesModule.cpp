// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPUtilitiesModule.h"
#include "GameplayTagContainer.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "IConcertClient.h"

#include "ShaderCore.h"
#include "UnrealEngine.h"
#include "VPRolesSubsystem.h"
#include "Interfaces/IPluginManager.h"

DEFINE_LOG_CATEGORY(LogVPUtilities);

namespace UE::VPUtilities::Private
{

bool EvaluateRole(const FGameplayTagContainer& Container)
{
	UVirtualProductionRolesSubsystem* VPRolesSubsystem = GEngine->GetEngineSubsystem<UVirtualProductionRolesSubsystem>();
	if (!VPRolesSubsystem)
	{
		return false;
	}

	for (auto It = Container.CreateConstIterator(); It; ++It)
	{
		if (VPRolesSubsystem->HasActiveRole(*It))
		{
			return true;
		}
	}
	return false;
}

}
/**
 * Implements the Concert Client module
 */
class FVPUtilitiesModule : public IModuleInterface
{
public:

	virtual void StartupModule() override
	{
		UE::ConcertClient::VPRoleEvaluator().BindStatic(
			&UE::VPUtilities::Private::EvaluateRole );
	}

	virtual void ShutdownModule() override
	{
	}

private:
};

IMPLEMENT_MODULE(FVPUtilitiesModule, VPUtilities)
