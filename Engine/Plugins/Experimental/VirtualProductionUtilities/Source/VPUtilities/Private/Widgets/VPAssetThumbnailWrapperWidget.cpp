// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/VPAssetThumbnailWrapperWidget.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#if WITH_EDITOR
#include "Components/AssetThumbnailWidget.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#endif

#define LOCTEXT_NAMESPACE "UVPAssetThumbnailWrapperWidget"

void UVPAssetThumbnailWrapperWidget::SetEditorThumbnailResolution(const FIntPoint& NewResolution)
{
#if WITH_EDITOR
	if (AssetWidget)
	{
		ExactCast<UAssetThumbnailWidget>(AssetWidget)->SetResolution(NewResolution);
	}
#endif
}

FIntPoint UVPAssetThumbnailWrapperWidget::GetEditorThumbnailResolution() const
{
#if WITH_EDITOR
	if (AssetWidget)
	{
		return ExactCast<UAssetThumbnailWidget>(AssetWidget)->GetResolution();
	}
#endif
	return FIntPoint::ZeroValue;
}

UObject* UVPAssetThumbnailWrapperWidget::GetEditorAssetWidget() const
{
#if WITH_EDITOR
	return AssetWidget;
#else
	return nullptr;
#endif
}

void UVPAssetThumbnailWrapperWidget::SetAsset(const FAssetData& AssetData)
{
#if WITH_EDITOR
	if (AssetWidget)
	{
		ExactCast<UAssetThumbnailWidget>(AssetWidget)->SetAsset(AssetData);
	}
#endif
}

void UVPAssetThumbnailWrapperWidget::SetAssetByObject(UObject* Object)
{
#if WITH_EDITOR
	if (AssetWidget)
	{
		ExactCast<UAssetThumbnailWidget>(AssetWidget)->SetAssetByObject(Object);
	}
#endif
}

void UVPAssetThumbnailWrapperWidget::SetFallbackBrush(const FSlateBrush& NewFallbackBrush)
{
	FallbackBrush = NewFallbackBrush;
	UpdateNativeAssetThumbnailWidget();
}

void UVPAssetThumbnailWrapperWidget::SetDisplayMode(EAssetThumbnailDisplayMode Mode)
{
#if WITH_EDITOR
	DisplayMode = Mode;
	UpdateNativeAssetThumbnailWidget();
#endif
}

void UVPAssetThumbnailWrapperWidget::SynchronizeProperties()
{
	Super::SynchronizeProperties();
#if WITH_EDITOR
	if (AssetWidget)
	{
		ExactCast<UAssetThumbnailWidget>(AssetWidget)->SynchronizeProperties();
	}
#endif
	UpdateNativeAssetThumbnailWidget();
}

void UVPAssetThumbnailWrapperWidget::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

#if WITH_EDITOR
	if (AssetWidget)
	{
		ExactCast<UAssetThumbnailWidget>(AssetWidget)->ReleaseSlateResources(bReleaseChildren);
		AssetWidget = nullptr;
	}
#endif
	Fallback.Reset();
	Content.Reset();
}

TSharedRef<SWidget> UVPAssetThumbnailWrapperWidget::RebuildWidget()
{
#if WITH_EDITOR
	AssetWidget = NewObject<UAssetThumbnailWidget>(this);
#endif
	
	Content = SNew(SWidgetSwitcher)
		.WidgetIndex(0)
#if WITH_EDITOR
		+SWidgetSwitcher::Slot()
		[
			ExactCast<UAssetThumbnailWidget>(AssetWidget)->RebuildWidget()
		]
#endif
	+SWidgetSwitcher::Slot()
		[
			SAssignNew(Fallback, SImage)
		];
	
	UpdateNativeAssetThumbnailWidget();
	return Content.ToSharedRef();
}

#if WITH_EDITOR
const FText UVPAssetThumbnailWrapperWidget::GetPaletteCategory()
{
	return LOCTEXT("VirtualProduction", "Virtual Production");
}
#endif

void UVPAssetThumbnailWrapperWidget::UpdateNativeAssetThumbnailWidget()
{
	Fallback->SetImage(&FallbackBrush);

#if WITH_EDITOR
	Content->SetActiveWidgetIndex(static_cast<int32>(DisplayMode));
#endif
}

#undef LOCTEXT_NAMESPACE