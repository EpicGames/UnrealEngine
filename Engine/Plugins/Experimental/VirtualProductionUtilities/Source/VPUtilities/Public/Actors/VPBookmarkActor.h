// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VPViewportTickableActorBase.h"
#include "UObject/UObjectGlobals.h"
#include "VPBookmark.h"
#include "IVPBookmarkProvider.h"
#include "IVPInteraction.h"
#include "CineCameraComponent.h"
#include "VPBookmarkActor.generated.h"

class UStaticMeshComponent;
class UTextRenderComponent;
class USplineMeshComponent;
class UMaterialInstanceDynamic;
class UMaterial;
class UCineCaptureComponent2D;

UCLASS(HideCategories=(Rendering, Lighting, HLOD, Mobile, Navigation, RayTracing, TextureStreaming), CollapseCategories)
class VPUTILITIES_API AVPBookmarkActor : public AVPViewportTickableActorBase, public IVPInteraction, public IVPBookmarkProvider
{
	GENERATED_BODY()

public:

	AVPBookmarkActor(const FObjectInitializer& ObjectInitializer);

	/** Mesh Representation in the world */
	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Category = "Components")
	TObjectPtr<UStaticMeshComponent> BookmarkMeshComponent;

	/** Textrender to display bookmark name */
	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Category = "Components")
	TObjectPtr<UTextRenderComponent> NameTextRenderComponent;

	/** Splinemesh */
	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Category = "Components")
	TObjectPtr<USplineMeshComponent> SplineMeshComponent;

	/**CineCamera */
	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Category = "Components")
	TObjectPtr<UCineCameraComponent> CameraComponent;

	/**CineCaptureComponent*/
	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Category = "Components")
	TObjectPtr<UCineCaptureComponent2D> SceneCaptureComponent;

	/** Color of Bookmark in MU Session */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Bookmark", meta = (MultiLine = "true", ExposeOnSpawn = "true"))
	FLinearColor BookmarkColor;

	/** Favorite status set by a user */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Bookmark")
	int32 FavoriteLevel = 0;

	/** Whether this was flagged by a user */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Bookmark")
	bool bIsFlagged = false;

	/** DateTime to store creation time */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Bookmark")
	FDateTime Timestamp;

	/** Reference to Editor Bookmark UObject*/
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = "Bookmark", meta = (MultiLine = "true"))
	TObjectPtr<UVPBookmark> BookmarkObject;

	/** Bool to determine if this bookmark should be designated the home location */
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = "Bookmark", meta = (MultiLine = "true"))
	bool IsHome;

	/** Texture reference to store render of viewpoint  */
	UPROPERTY(BlueprintReadWrite, VisibleAnywhere, Category = "Snapshot")
	TObjectPtr<UTexture2D> SnapshotTexture;

	/**Update the mesh color and BookmarkColor variable. Intended for use with multiuser initialization*/
	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Default")
	void UpdateBookmarkColor(FLinearColor Color);

	UPROPERTY(BlueprintReadWrite, Category = "Default")
	TObjectPtr<UMaterialInstanceDynamic> DynamicMaterial;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "Default")
	TObjectPtr<UMaterialInterface> BookmarkMaterial;

	/** Update SnapshotTexture captured via SceneCaptureComponent*/
	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Snapshot")
	void CaptureSnapshot();

	/** Update Timestamp property with current time*/
	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Bookmark")
	void UpdateTimestamp();

	//VPBookmark Interface events

	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Bookmarks")
	void OnBookmarkActivation_Implementation(UVPBookmark* Bookmark, bool bActivate);

	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Bookmarks")
	void OnBookmarkChanged_Implementation(UVPBookmark* Bookmark);

	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Bookmarks")
	void UpdateBookmarkSplineMeshIndicator_Implementation();

	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Bookmarks")
	void HideBookmarkSplineMeshIndicator_Implementation();

	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Bookmarks")
	void GenerateBookmarkName_Implementation();


	//VPInteraction Interface Events

	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Interaction")
	void OnActorDroppedFromCarry_Implementation();

	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Interaction")
	void OnActorSelectedForTransform_Implementation();

	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Interaction")
	void OnActorDroppedFromTransform_Implementation();

	//Overrides

	virtual void PostRegisterAllComponents() override;

	virtual void Tick(float DeltaSeconds) override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	
private:

	FRotator BookmarkRotation; //Actor's rotation

	UPROPERTY()
	TObjectPtr<UStaticMesh> BookmarkStaticMesh; //Mesh to use main static mesh component

	UPROPERTY()
	TObjectPtr<UMaterial> TextMaterial; 

	UPROPERTY()
	TObjectPtr<UStaticMesh> SplineStaticMesh;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> SplineMaterial;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> SplineMaterialInstance;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> LabelMaterialInstance;
};
