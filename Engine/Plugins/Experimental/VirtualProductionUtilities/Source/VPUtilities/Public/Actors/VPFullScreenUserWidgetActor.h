// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Info.h"
#include "VPFullScreenUserWidgetActor.generated.h"


class UViewportWidgetOverlay;

/**
 * Widgets are first rendered to a render target, then that render target is displayed in the world.
 */
UCLASS(Blueprintable, ClassGroup = "UserInterface", HideCategories = (Actor, Input, Movement, Collision, Rendering, Transformation, LOD), ShowCategories = ("Input|MouseInput", "Input|TouchInput"))
class VPUTILITIES_API AFullScreenUserWidgetActor : public AInfo
{
	GENERATED_BODY()

public:
	AFullScreenUserWidgetActor(const FObjectInitializer& ObjectInitializer);

	//~ Begin AActor interface
	virtual void PostInitializeComponents() override;
	virtual void PostLoad() override;
	virtual void PostActorCreated() override;
	virtual void Destroyed() override;

	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	virtual void Tick(float DeltaSeconds) override;

#if WITH_EDITOR
		virtual bool ShouldTickIfViewportsOnly() const override { return true; }
#endif //WITH_EDITOR
	//~ End AActor Interface

	/* Get a pointer to the inner user widget  */
	UFUNCTION(BlueprintCallable, Category = "User Interface")
	UUserWidget* GetUserWidget() const;

private:
	void RequestEditorDisplay();
	void RequestGameDisplay();

protected:
	/** */
	UPROPERTY(VisibleAnywhere, Instanced, NoClear, Category = "User Interface", meta = (ShowOnlyInnerProperties))
	TObjectPtr<UViewportWidgetOverlay> ScreenUserWidget;

#if WITH_EDITORONLY_DATA
	/** Display requested and will be executed on the first frame because we can't call BP function in the loading phase */
	bool bEditorDisplayRequested;
#endif
};
