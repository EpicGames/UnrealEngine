// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/PostProcessVolume.h"
#include "VPPassthroughPostProcessVolume.generated.h"

/**
 * AVPPassthroughPostProcessVolume is derived from APostProcessVolume.
 * Its function is to provide a Post Process Volume offers a clean passthrough of colors into display space, with no additional post-processing.
 * This is done by customizing some of the default values.
 */
UCLASS(MinimalAPI, DisplayName="VP Passthrough Post Process Volume")
class AVPPassthroughPostProcessVolume : public APostProcessVolume
{
	GENERATED_BODY()

public:
	AVPPassthroughPostProcessVolume()
	{
		BlendWeight = 1.f;
		bUnbound = true;

		Settings.bOverride_BloomIntensity = 1;
		Settings.BloomIntensity = 0.f;

		Settings.bOverride_AutoExposureMethod = 1;
		Settings.AutoExposureMethod = EAutoExposureMethod::AEM_Manual;

		Settings.bOverride_AutoExposureBias = 1;
		Settings.AutoExposureBias = 0.f;

		Settings.bOverride_AutoExposureApplyPhysicalCameraExposure = 1;
		Settings.AutoExposureApplyPhysicalCameraExposure = 0;

		Settings.bOverride_LocalExposureHighlightContrastScale = 1;
		Settings.LocalExposureHighlightContrastScale = 1.f;

		Settings.bOverride_LocalExposureShadowContrastScale = 1;
		Settings.LocalExposureShadowContrastScale = 1.f;

		Settings.bOverride_LocalExposureDetailStrength = 1;
		Settings.LocalExposureDetailStrength = 1.f;

		Settings.bOverride_LensFlareIntensity = 1;
		Settings.LensFlareIntensity = 0.f;

		Settings.bOverride_ExpandGamut = 1;
		Settings.ExpandGamut = 0.f;

		Settings.bOverride_ToneCurveAmount = 1;
		Settings.ToneCurveAmount = 0.f;

		Settings.bOverride_BlueCorrection = 1;
		Settings.BlueCorrection = 0.f;

		Settings.bOverride_VignetteIntensity = 1;
		Settings.VignetteIntensity = 0.f;

		Settings.bOverride_FilmGrainIntensity = 1;
		Settings.FilmGrainIntensity = 0.0f;

		Settings.bOverride_SceneFringeIntensity = 1;
		Settings.SceneFringeIntensity = 0.0f;

		Settings.bOverride_LumenFinalGatherScreenTraces = 1;
		Settings.LumenFinalGatherScreenTraces = 0;

		Settings.bOverride_LumenReflectionsScreenTraces = 1;
		Settings.LumenReflectionsScreenTraces = 0;
	}
};
