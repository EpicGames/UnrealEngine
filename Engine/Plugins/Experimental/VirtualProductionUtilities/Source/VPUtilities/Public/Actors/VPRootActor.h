// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Components/BoxComponent.h"
#include "Engine/AssetUserData.h"
#include "GameFramework/Actor.h"
#include "VPRootActor.generated.h"


class ACineCameraActor;
class UArrowComponent;
class UBillboardComponent;
class UMeshComponent;


/**
 * AVPRootActor
 */
UCLASS(meta = (DisplayName = "VP Root Actor"), hidecategories=(Actor, Collision, Input, HLOD, Replication))
class VPUTILITIES_API AVPRootActor : public AActor
{
	GENERATED_BODY()

public:
	AVPRootActor(const FObjectInitializer& ObjectInitializer);

public:
	/** Actor used to show the size of the scene. */
	UPROPERTY(EditDefaultsOnly, Category = "Virtual Production")
	TObjectPtr<UMeshComponent> RealWorldSceneRepresentation;

	/** Actor used to show the size of the scene. */
	UPROPERTY(EditAnywhere, Category = "Virtual Production")
	TObjectPtr<ACineCameraActor> CinematicCamera;

private:
	UPROPERTY()
	bool bAreComponentsVisible;

#if WITH_EDITORONLY_DATA
protected:
	/** Billboard used to see the scene in the editor */
	UPROPERTY()
	TObjectPtr<UBillboardComponent> SpriteComponent;

	/** Arrow used to see the orientation of the scene in the editor */
	UPROPERTY()
	TObjectPtr<UArrowComponent> ArrowComponent;

public:
	UPROPERTY(EditAnywhere, Category = "Virtual Production")
	bool bMoveLevelWithActor;

	bool bReentrantPostEditMove;
#endif

public:
	/** Get the current camera used by the virtual production. */
	UFUNCTION(BlueprintNativeEvent, Category = "Virtual Production")
	ACineCameraActor* GetCineCameraActor() const;

	virtual void BeginPlay() override;

#if WITH_EDITOR
	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Virtual Production")
	void ToggleComponentsVisibility();

	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Virtual Production")
	void MoveLevelToRootActor();

	virtual void CheckForErrors() override;
	virtual void PostEditMove(bool bFinished) override;
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif

private:
	void SetComponentsVisibility(bool bVisible);
};


/**
 * UVPWorldAssetUserData
 */
UCLASS(MinimalAPI)
class UVPWorldAssetUserData : public UAssetUserData
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Virtual Production")
	TLazyObjectPtr<AVPRootActor> LastSelectedRootActor;
};
