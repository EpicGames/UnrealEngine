// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "UObject/UObjectGlobals.h"

#include "VPViewportTickableActorBase.generated.h"

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EVPViewportTickableFlags : uint8
{
	Editor			= 1 << 0,
	Game			= 1 << 1,
	EditorPreview	= 1 << 2,
	GamePreview		= 1 << 3,
};
ENUM_CLASS_FLAGS(EVPViewportTickableFlags)

/**
 * Actor that tick in the Editor viewport with the event EditorTick.
 */
UCLASS(Abstract)
class VPUTILITIES_API AVPViewportTickableActorBase : public AActor
{
	GENERATED_BODY()

public:
	AVPViewportTickableActorBase(const FObjectInitializer& ObjectInitializer);

	UFUNCTION(BlueprintNativeEvent, CallInEditor, BlueprintCallable, Category = "Tick")
	void EditorTick(float DeltaSeconds);

	UFUNCTION(BlueprintNativeEvent, CallInEditor, BlueprintCallable, Category = "Utilities")
	void EditorDestroyed();

	/** Sets the LockLocation variable to disable movement from the translation gizmo */
	UFUNCTION(CallInEditor, BlueprintCallable, Category = "Editor")
	void EditorLockLocation(bool bSetLockLocation);

	/**
	 * Where the actor should be ticked.
	 * Editor = Tick in the editor viewport. Use the event EditorTick.
	 * Game = Tick in game even if we are only ticking the viewport. Use the event Tick.
	 * Preview = Tick if the actor is present in any editing tool like Blueprint or Material graph. Use EditorTick.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Actor Tick", meta = (Bitmask, BitmaskEnum = "/Script/VPUtilities.EVPViewportTickableType"))
	EVPViewportTickableFlags ViewportTickType;

	/** If true, actor is ticked even if TickType==LEVELTICK_ViewportsOnly */
	virtual bool ShouldTickIfViewportsOnly() const override;
	virtual void Tick(float DeltaSeconds) override;
	virtual void Destroyed() override;
};
