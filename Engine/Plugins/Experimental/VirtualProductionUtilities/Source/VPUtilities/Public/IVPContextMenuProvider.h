// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "IVPContextMenuProvider.generated.h"


class UVPContextMenu;

UINTERFACE(BlueprintType)
class UVPContextMenuProvider : public UInterface
{
	GENERATED_BODY()
};


class IVPContextMenuProvider : public IInterface
{
	GENERATED_BODY()

public:
	UE_DEPRECATED(5.5,"Code will be removed from UE5.7")
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, CallInEditor, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	void OnCreateContextMenu();
};
