// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "IVPInteraction.generated.h"

class UVirtualScoutingInteraction;

// This class does not need to be modified.

UINTERFACE(BlueprintType)
class UVPInteraction : public UInterface
{
	GENERATED_BODY()
};

class IVPInteraction : public IInterface  
{
	GENERATED_BODY()

	// Add interface functions to this class. This is the class that will be inherited to implement this interface.
public:

	/** Called when actor is dropped from Carry state by the VREd interactor */
	UE_DEPRECATED(5.5,"Code will be removed from UE5.7")
	UFUNCTION(BlueprintNativeEvent, CallInEditor, BlueprintCallable, Category = "VirtualScouting|Interaction", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	void OnActorDroppedFromCarry();

	/** Called when an actor is selected by the VREd interactor */
	UE_DEPRECATED(5.5,"Code will be removed from UE5.7")
	UFUNCTION(BlueprintNativeEvent, CallInEditor, BlueprintCallable, Category = "VirtualScouting|Interaction", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	void OnActorSelectedForTransform();

	/** Called when dropped by VRTool */
	UE_DEPRECATED(5.5,"Code will be removed from UE5.7")
	UFUNCTION(BlueprintNativeEvent, CallInEditor, BlueprintCallable, Category = "VirtualScouting|Interaction", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	void OnActorDroppedFromTransform();
};