// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "Actors/VPViewportTickableActorBase.h"
#include "GameplayTagContainer.h"
#include "VPBookmarkContext.h"

#include "VPBlueprintLibrary.generated.h"


class UVPBookmark;
class AVPBookmarkActor;

/**
 * Functionality added to prototype the VR scouting tools
 */
UCLASS()
class VPUTILITIES_API UVPBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	
	/** Refresh the desktop 3D viewport so that it updates changes even when not set to 'Realtime' */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production")
	static void Refresh3DEditorViewport();

	/** Spawn a virtual production tickable actor */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production")
	static AVPViewportTickableActorBase* SpawnVPTickableActor(UObject* ContextObject, const TSubclassOf<AVPViewportTickableActorBase> ActorClass, const FVector Location, const FRotator Rotation);

	/** Spawn a virtual production bookmark */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production")
	static AActor* SpawnBookmarkAtCurrentLevelEditorPosition(const TSubclassOf<AActor> ActorClass, const FVPBookmarkCreationContext CreationContext, const FVector Offset, const bool bFlattenRotation = true);
	
	/** Jump to a virtual production bookmark */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production")
	static bool JumpToBookmarkInLevelEditor(const UVPBookmark* Bookmark);

	/** The machine role(s) in a virtual production context. */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production")
	static FGameplayTagContainer GetVirtualProductionRole();

	/** Get the location of the 2D viewport camera */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production")
	static FTransform GetEditorViewportTransform();

	/** Get the location of the VR HMD */
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta = (DeprecatedFunction, DeprecationMessage = "VR Editor mode is deprecated; use the XR Creative Framework plugin instead."))
	static FTransform GetEditorVRHeadTransform();

	/** Get the VR room transform (the playable area shown as a wireframe cage on Vive and Rift */
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta = (DeprecatedFunction, DeprecationMessage = "VR Editor mode is deprecated; use the XR Creative Framework plugin instead."))
	static FTransform GetEditorVRRoomTransform();

	/** Set the VR grab speed cvar */
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta = (DeprecatedFunction, DeprecationMessage = "VR Editor mode is deprecated; use the XR Creative Framework plugin instead."))
	static void SetGrabSpeed(const float Speed);

	/** Get whether the user is in editor VR mode */
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta = (DeprecatedFunction, DeprecationMessage = "VR Editor mode is deprecated; use the XR Creative Framework plugin instead."))
	static bool IsVREditorModeActive();

	/** Get the hitlocation of the interaction controller's laser pointer, in world space */
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta = (DeprecatedFunction, DeprecationMessage = "VR Editor mode is deprecated; use the XR Creative Framework plugin instead."))
	static FVector GetVREditorLaserHoverLocation();

	/** Get the location of the left interactor, in world space */
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta = (DeprecatedFunction, DeprecationMessage = "VR Editor mode is deprecated; use the XR Creative Framework plugin instead."))
	static FVector GetLeftInteractorLocation();

	/** Trigger an UnrealEd Undo */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production")
	static bool EditorUndo();

	/** Trigger an UnrealEd Redo */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production")
	static bool EditorRedo();

	/** Trigger an UnrealEd Duplicate */
	UE_DEPRECATED(5.7, "Use UEditorActorSubsystem::DuplicateSelectedActors instead.")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta = (DeprecatedFunction, DeprecationMessage = "Use UEditorActorSubsystem::DuplicateSelectedActors instead."))
	static bool EditorDuplicate();

	/** Trigger an UnrealEd Delete */
	UE_DEPRECATED(5.7, "Use UEditorActorSubsystem::DeleteSelectedActors instead.")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta = (DeprecatedFunction, DeprecationMessage = "Use UEditorActorSubsystem::DeleteSelectedActors instead."))
	static bool EditorDeleteSelectedObjects();

	/**
	 * Wrapper around UUnrealEditorSubsystem::GetEditorWorld.
	 * Used because you can't get the subsystem without being an editor utility actor.
	 */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production")
	static UWorld* GetEditorWorld();

	UFUNCTION(BlueprintCallable, Category = "VPBookmarks")		
	static void VPBookmarkSplineMeshIndicatorSetStartAndEnd(USplineMeshComponent* SplineMesh);

	UFUNCTION(BlueprintCallable, Category = "VPBookmarks")
	static void VPBookmarkSplineMeshIndicatorDisable(USplineMeshComponent* SplineMesh);

	/** Sort actors by actor label*/
	UFUNCTION(BlueprintCallable, Category = "VPBookmarks")
	static void SortActorsByName(UPARAM(Ref) TArray<AActor*>& Actors, bool Ascending = true);

	/** Sort VPBookmarkActors by Timestamp parameter*/
	UFUNCTION(BlueprintCallable, Category = "VPBookmarks")
	static void SortVPBookmarkActorsByTimestamp(UPARAM(Ref) TArray<AVPBookmarkActor*>& Actors, bool Ascending = true);
	
};