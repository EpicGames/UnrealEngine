// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/SplineComponent.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "VPCameraBlueprintLibrary.generated.h"


class ACameraRig_Rail;

UENUM(BlueprintType)
enum class UE_DEPRECATED(5.5,"Code will be removed from UE5.7") EVPCameraRigSpawnLinearApproximationMode : uint8
{
	None			UMETA(Display = "No Approximation"),	// We won't do linear approximation, instead using the Spline as constructed initially.
	Density,												// LinearApproximationParam will be used as a density value 
	IntegrationStep,										// LinearApproximationParam will be used as the Integration step in Unreal Units.
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
/** Parameters used to custom the CameraRig that's created. */
USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.5,"Code will be removed from UE5.7") FVPCameraRigSpawnParams
{
	GENERATED_BODY();

	FVPCameraRigSpawnParams();

public:
	/** Use world space (as opposed to local space) for points. */
	UE_DEPRECATED(5.5,"This property will be removed from UE5.7")
	UPROPERTY(Transient, BlueprintReadWrite, Category="Camera Rig",  meta=(DeprecatedProperty, DeprecationMessage="This property deprecated"))
	bool bUseWorldSpace;

	/**
	 * Use the first vector of input as the spawn transform.
	 * Causes RigTransform to be completely ignored.
	 */
	UE_DEPRECATED(5.5,"This property will be removed from UE5.7")
	UPROPERTY(Transient, BlueprintReadWrite, Category = "Camera Rig",  meta=(DeprecatedProperty, DeprecationMessage="This property deprecated"))
	bool bUseFirstPointAsSpawnLocation;

	/**
	 * Causes a linear approximation of the spline points to be generated instead
	 * of relying purely on the passed in points / curves.
	 */
	UE_DEPRECATED(5.5,"This property will be removed from UE5.7")
	UPROPERTY(Transient, BlueprintReadWrite, Category = "Camera Rig",  meta=(DeprecatedProperty, DeprecationMessage="This property deprecated"))
	EVPCameraRigSpawnLinearApproximationMode LinearApproximationMode;

	/**
	 * This is only used if LinearApproximationMode is not None.
	 * When mode is Density:
	 * See FSplinePositionLinearApproximation::Build.
	 *
	 * When mode is IntegrationStep:
	 * Integration step (in CM) between approximation points. Decreasing this value will
	 * increase the number of spline points and will therefore increase the accuracy
	 * (at the cost of increased complexity).
	 */
	UE_DEPRECATED(5.5,"This property will be removed from UE5.7")
	UPROPERTY(Transient, BlueprintReadWrite, Category = "Camera Rig",  meta=(DeprecatedProperty, DeprecationMessage="This property deprecated"))
	float LinearApproximationParam;
};

UCLASS()
class UE_DEPRECATED(5.5,"Code will be removed from UE5.7") VPUTILITIES_API UVPCameraBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UE_DEPRECATED(5.5,"Code will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static ACameraRig_Rail* SpawnDollyTrackFromPoints(UObject* WorldContextObject, UPARAM(ref) const TArray<FTransform>& Points, ESplinePointType::Type InterpType = ESplinePointType::Linear);

	UE_DEPRECATED(5.5,"Code will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static ACameraRig_Rail* SpawnDollyTrackFromPointsSmooth(UObject* WorldContextObject, UPARAM(ref) const TArray<FTransform>& Points, ESplinePointType::Type InterpType = ESplinePointType::Linear);

	UE_DEPRECATED(5.5,"Code will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static ACameraRig_Rail* SpawnCameraRigFromActors(UObject* WorldContextObject, const FTransform& RigTransform, const TArray<AActor*>& Actors, const FVPCameraRigSpawnParams& Params);

	UE_DEPRECATED(5.5,"Code will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static ACameraRig_Rail* SpawnCameraRigFromPoints(UObject* WorldContextObject, const FTransform& RigTransform, const TArray<FVector>& Points, const FVPCameraRigSpawnParams& Params);

	UE_DEPRECATED(5.5,"Code will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static ACameraRig_Rail* SpawnCameraRigFromSelectedActors(UObject* WorldContextObject, const FTransform& RigTransform, const FVPCameraRigSpawnParams& Params);
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS