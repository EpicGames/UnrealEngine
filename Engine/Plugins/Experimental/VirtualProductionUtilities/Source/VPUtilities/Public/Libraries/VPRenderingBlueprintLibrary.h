// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "SceneViewExtensionContext.h"

#include "VPRenderingBlueprintLibrary.generated.h"

/**
 * Blueprint function library supporting Virtual Production rendering use cases.
 */
UCLASS()
class VPUTILITIES_API UVPRenderingBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** 
	 * Returns a functor to activate a scene view extension when the FViewport in the SceneViewExtensionContext is selectively:
	 * - PIE
	 * - SIE
	 * - Editor's active
	 * - Game's primary
	 * If it is none of the above, it emits no opinion about activating the Scene View Extension.
	 */
	UFUNCTION(BlueprintPure, Category = "Virtual Production|Rendering")
	static void GenerateSceneViewExtensionIsActiveFunctorForViewportType(
		FSceneViewExtensionIsActiveFunctor& OutIsActiveFunction,
		bool bPIE = false,
		bool bSIE = false,
		bool bEditorActive = false,
		bool bGamePrimary = false
	);

};