// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/SoftObjectPath.h"
#include "VPBookmarkSettings.generated.h"

UCLASS(Config = VirtualProductionUtilities)
class VPUTILITIES_API UVPBookmarkSettings : public UObject
{
	GENERATED_BODY()
		//~ Bookmark assets //
public:
	/** Bookmark mesh to use for Bookmark actor */
	UPROPERTY(config, EditAnywhere, Category = "VP Bookmark", meta = (AllowedClasses = "/Script/Engine.StaticMesh", DisplayName = "Bookmark Mesh"))
	FSoftObjectPath BookmarkMeshPath;

	/** Bookmark mesh material to use for Bookmark actor */
	UPROPERTY(config, EditAnywhere, Category = "VP Bookmark", meta = (AllowedClasses = "/Script/Engine.MaterialInterface", DisplayName = "Bookmark Mesh Material"))
	FSoftObjectPath BookmarkMaterialPath;

	/** Bookmark spline mesh to use for Bookmark actor */
	UPROPERTY(config, EditAnywhere, Category = "VP Bookmark", meta = (AllowedClasses = "/Script/Engine.StaticMesh", DisplayName = "Bookmark Spline Mesh"))
	FSoftObjectPath BookmarkSplineMeshPath;

	/** Bookmark spline mesh material to use for Bookmark actor */
	UPROPERTY(config, EditAnywhere, Category = "VP Bookmark", meta = (AllowedClasses = "/Script/Engine.MaterialInterface", DisplayName = "Bookmark Spline Material"))
	FSoftObjectPath BookmarkSplineMeshMaterialPath;

	/** Bookmark label material class to use for Bookmark actor */
	UPROPERTY(config, EditAnywhere, Category = "VP Bookmark", meta = (AllowedClasses = "/Script/Engine.MaterialInterface", DisplayName = "Bookmark Label Material"))
	FSoftObjectPath BookmarkLabelMaterialPath;
};
