// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/GameMode.h"

#include "VPGameMode.generated.h"


class AActor;
class AController;
class AVPRootActor;

/**
 * 
 */
UCLASS(abstract)
class VPUTILITIES_API AVPGameMode : public AGameMode
{
	GENERATED_BODY()

private:
	UPROPERTY(transient, BlueprintGetter = "GetRootActor", Category = "Virtual Production")
	TObjectPtr<AVPRootActor> RootActor;

public:
	UFUNCTION(BlueprintGetter, Category = "Virtual Production")
	AVPRootActor* GetRootActor() const { return RootActor; }

public:
	virtual AActor* ChoosePlayerStart_Implementation(AController* Player) override;
};
