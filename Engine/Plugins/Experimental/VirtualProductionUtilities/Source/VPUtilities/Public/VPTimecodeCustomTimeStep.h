// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

  #pragma once
  #include "TimecodeCustomTimeStep.h"

/**
 * Deprecated in 5.8. Moved to the TimeManagement module as UTimecodeCustomTimeStep.
 */

  using UVPTimecodeCustomTimeStep UE_DEPRECATED(5.8, "UVPTimecodeCustomTimeStep has moved to the TimeManagement module as UTimecodeCustomTimeStep. Please update references.") = UTimecodeCustomTimeStep;