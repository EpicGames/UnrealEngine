// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

UE_DEPRECATED_HEADER(5.8, "Include ViewportWidgetOverlay_PostProcessWithSVE.h instead")
#include "Misc/ViewportWidgetOverlay_PostProcessWithSVE.h"

struct UE_DEPRECATED(5.8, "Use FViewportWidgetOverlay_PostProcessWithSVE instead.") FVPFullScreenUserWidget_PostProcessWithSVE;