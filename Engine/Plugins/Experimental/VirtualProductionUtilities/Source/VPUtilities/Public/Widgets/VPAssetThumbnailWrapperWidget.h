// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Components/Widget.h"
#include "VPAssetThumbnailWrapperWidget.generated.h"

class SImage;
class SWidgetSwitcher;

UENUM()
enum class EAssetThumbnailDisplayMode : uint8
{
	EditorThumbnail,
	FallbackImage
};

/**
 * Version of UAssetThumbnailWidget that compiles in packaged games.
 * In editor builds, a thumbnail widget is displayed.
 * In packaged builds, a fallback SImage is displayed.
 */
UCLASS(DisplayName = "Asset Thumbnail Widget (Editor & Game)")
class VPUTILITIES_API UVPAssetThumbnailWrapperWidget : public UWidget
{
	GENERATED_BODY()
public:

	/** Sets the resolution of the editor thumbnail */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	void SetEditorThumbnailResolution(const FIntPoint& NewResolution);
	UFUNCTION(BlueprintPure, Category = "Appearance")
	FIntPoint GetEditorThumbnailResolution() const;

	/**
	 * Gets the widget used for displaying in editor. Returns nullptr in non-editor builds.
	 * This type must be cast to UAssetThumbnailWidget manually (due to Unreal Header Tool restrictions).
	 */
	UFUNCTION(BlueprintPure, Category = "Appearance")
	UObject* GetEditorAssetWidget() const;
	
	/** Sets the asset to display. Has no effect in non-editor builds. */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	void SetAsset(const FAssetData& AssetData);
	/** Sets the asset to display. Has no effect in non-editor builds. */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	void SetAssetByObject(UObject* Object);

	/** Sets the fallback image to display. */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	void SetFallbackBrush(const FSlateBrush& NewFallbackBrush);
	UFUNCTION(BlueprintGetter, Category = "Appearance")
	FSlateBrush GetFallbackBrush() const { return FallbackBrush; }

	/** Sets the display mode. Only has an effect in editor builds because packaged games always display in FallbackImage mode. */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	void SetDisplayMode(EAssetThumbnailDisplayMode Mode = EAssetThumbnailDisplayMode::FallbackImage);

	//~ Begin UWidget Interface
	virtual void SynchronizeProperties() override;
	virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	virtual TSharedRef<SWidget> RebuildWidget() override;
#if WITH_EDITOR
	virtual const FText GetPaletteCategory() override;
#endif
	//~ End UWidget Interface
	
private:

	/** Fallback brush to draw */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetFallbackBrush, BlueprintSetter = SetFallbackBrush, Category = "Appearance")
	FSlateBrush FallbackBrush;

#if WITH_EDITORONLY_DATA
	/** The content for the editor version. Is an instance of UAssetThumbnailWidget but for non-editor builds UHT does not find the type so UObject here. */
	UPROPERTY(Instanced)
	TObjectPtr<UObject> AssetWidget;
	
	/** How the widget is supposed to be displayed. */
	UPROPERTY(EditAnywhere, BlueprintSetter = SetDisplayMode, AdvancedDisplay, Category = "Appearance")
	EAssetThumbnailDisplayMode DisplayMode = EAssetThumbnailDisplayMode::EditorThumbnail;
#endif
	
	/** The fallback image */
	TSharedPtr<SImage> Fallback;

	/** Holds the contents: either ThumbnailRenderer or a fallback SImage.*/
	TSharedPtr<SWidgetSwitcher> Content;
	
	/** Updates or creates ThumbnailRenderer and sets the contents of DisplayedWidget. */
	void UpdateNativeAssetThumbnailWidget();
};
