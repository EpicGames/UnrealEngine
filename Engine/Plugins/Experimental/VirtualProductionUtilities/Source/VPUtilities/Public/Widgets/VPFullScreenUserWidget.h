// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

UE_DEPRECATED_HEADER(5.8, "Include ViewportWidgetOverlay.h instead")
#include "ViewportWidgetOverlay.h"

enum class UE_DEPRECATED(5.8, "Use EViewportWidgetOverlay_DisplayType instead.") EVPWidgetDisplayType : uint8;
struct UE_DEPRECATED(5.8, "Use FViewportWidgetOverlay_Viewport instead.") FVPFullScreenUserWidget_Viewport;
class UE_DEPRECATED(5.8, "Use UViewportWidgetOverlay instead.") UVPFullScreenUserWidget;