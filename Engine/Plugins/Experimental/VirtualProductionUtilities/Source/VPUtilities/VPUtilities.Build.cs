// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VPUtilities : ModuleRules
{
	public VPUtilities(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
				"CinematicCamera",
				"CineCameraSceneCapture",
				"Core",
				"CoreUObject",
				"Engine",
				"GameplayTags",
				"TimeManagement",
				"UMG",
				"VPBookmark",
				"VPSettings"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"Projects",
				"Renderer",
				"RenderCore",
				"RHI",
				"Slate",
				"SlateCore",
				"ConcertClient",
				"VPRoles",
				"ViewportWidgetOverlay"
			}
		);

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Blutility",
					"EditorFramework",
					"LevelEditor",
					"UMGEditor",
					"UnrealEd",
					"ViewportInteraction",
					"VPBookmarkEditor",
					"VREditor",
				}
			);
		}
	}
}
