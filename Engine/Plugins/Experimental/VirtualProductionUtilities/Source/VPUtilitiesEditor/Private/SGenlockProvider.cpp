// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGenlockProvider.h"

#include "EditorFontGlyphs.h"
#include "Styling/AppStyle.h"
#include "Engine/Engine.h"
#include "Engine/EngineCustomTimeStep.h"
#include "FixedFrameRateCustomTimeStep.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/App.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"

#include "Widgets/Colors/SColorBlock.h"

#define LOCTEXT_NAMESPACE "GenlockProvider"

namespace GenlockProviderUtils 
{
	/**
	* Get a access to the Custom time step
	* @return A pointer to the Custom Time Step. Be careful the pointer can dangle.
	*/
	UEngineCustomTimeStep* GetCustomTimeStep();

	/**
	* Get a access to the Custom time step if it is a Fixed Custom Time Step
	* @return A pointer to the Fixed Custom Time Step. Be careful the pointer can dangle.
	*/
	UFixedFrameRateCustomTimeStep* GetFixedCustomTimeStep();
}

void SGenlockProvider::Tick(float DeltaTime)
{
	const UFixedFrameRateCustomTimeStep* FixedTimeStep = GenlockProviderUtils::GetFixedCustomTimeStep();
	if (FixedTimeStep && FixedTimeStep->GetSynchronizationState() == ECustomTimeStepSynchronizationState::Synchronized)
	{
		AvgIdleTime = FApp::GetIdleTime() * 0.2 + AvgIdleTime * 0.8;
		bIsAvgIdleTimeValid = true;
	}
	else
	{
		AvgIdleTime = 0.0;
		bIsAvgIdleTimeValid = false;
	}
}

TStatId SGenlockProvider::GetStatId() const
{
	return TStatId();
}

void SGenlockProvider::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SGenlockProvider::Construct(const FArguments & InArgs)
{
	AvgIdleTime = 0.0;
	bIsAvgIdleTimeValid = false;

	TSharedRef<SWidget> StateDisplay = ConstructStateDisplay();
	TSharedRef<SWidget> DesiredFPSWidget = ConstructDesiredFPS();

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0)
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, 4.f, 0.f)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.AutoWidth()
			[
				StateDisplay
			]
			+SHorizontalBox::Slot()
			.Padding(0)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SGenlockProvider::HandleGenlockSourceText)
			]
			+SHorizontalBox::Slot()
			.Padding(15,0,0,0)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			[
				DesiredFPSWidget
			]
		]
		+ SVerticalBox::Slot()
		.Padding(4.f)
		.VAlign(EVerticalAlignment::VAlign_Top)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(2.f, 6.f)
			//.AutoWidth()
			[
				SNew(SProgressBar)
				.BorderPadding(FVector2D::ZeroVector)
				.Percent(this, &SGenlockProvider::GetFPSFraction)
				.FillColorAndOpacity(FSlateColor(FLinearColor(0.0f, 1.0f, 1.0f)))
				.Visibility(this, &SGenlockProvider::HandleDesiredFPSVisibility)
				.ToolTipText(this, &SGenlockProvider::GetFPSTooltip)
			]
		]
	];
}

TSharedRef<SWidget> SGenlockProvider::ConstructDesiredFPS() const
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Right)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(1)
			[
				SNew(STextBlock)
				.Text(this, &SGenlockProvider::HandleDesiredFPSText)
				.Visibility(this, &SGenlockProvider::HandleDesiredFPSVisibility)
			]
		];
}

TSharedRef<SWidget> SGenlockProvider::ConstructStateDisplay() const
{
	return SNew(STextBlock)
		.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
		.Text(this, &SGenlockProvider::HandleStateText)
		.ColorAndOpacity(this, &SGenlockProvider::HandleStateColorAndOpacity);
}


FText SGenlockProvider::HandleStateText() const
{
	UEngineCustomTimeStep*  CustomTimeStep = GenlockProviderUtils::GetCustomTimeStep();

	if (CustomTimeStep)
	{
		switch (CustomTimeStep->GetSynchronizationState())
		{
		case ECustomTimeStepSynchronizationState::Synchronized:
			return FEditorFontGlyphs::Clock_O;
		case ECustomTimeStepSynchronizationState::Synchronizing:
			return FEditorFontGlyphs::Hourglass_O;
		case ECustomTimeStepSynchronizationState::Error:
		case ECustomTimeStepSynchronizationState::Closed:
		default:
			return FEditorFontGlyphs::Ban;
		}
	}
	return FEditorFontGlyphs::Exclamation;
}

FSlateColor SGenlockProvider::HandleStateColorAndOpacity() const
{
	UEngineCustomTimeStep*  CustomTimeStep = GenlockProviderUtils::GetCustomTimeStep();

	if (CustomTimeStep)
	{
		switch (CustomTimeStep->GetSynchronizationState())
		{
		case ECustomTimeStepSynchronizationState::Closed:
		case ECustomTimeStepSynchronizationState::Error:
			return FLinearColor::Red;
		case ECustomTimeStepSynchronizationState::Synchronized:
			return FLinearColor::Green;
		case ECustomTimeStepSynchronizationState::Synchronizing:
			return FLinearColor::Yellow;
		}
	}
	//If there is no custom time step
	return FSlateColor::UseForeground();
}

FText SGenlockProvider::HandleDesiredFPSText() const
{
	const UFixedFrameRateCustomTimeStep* FixedTimeStep = GenlockProviderUtils::GetFixedCustomTimeStep();
	return FixedTimeStep ? FixedTimeStep->GetFixedFrameRate().ToPrettyText() : FText();
}

FText SGenlockProvider::HandleGenlockSourceText() const
{
	if (UEngineCustomTimeStep* CustomTimeStep = GenlockProviderUtils::GetCustomTimeStep())
	{
		return FText::FromName(CustomTimeStep->GetFName());
	}
	return LOCTEXT("NoGenlockText", "No Genlock");
}

TOptional<float> SGenlockProvider::GetFPSFraction() const
{
	if (bIsAvgIdleTimeValid)
	{
		if (UFixedFrameRateCustomTimeStep* FixedTimeStep = GenlockProviderUtils::GetFixedCustomTimeStep())
		{
			const double DeciamlFixeFrameRate = FixedTimeStep->GetFixedFrameRate().AsInterval();
			const double Difference = DeciamlFixeFrameRate - AvgIdleTime;
			return Difference / DeciamlFixeFrameRate;
		}
	}
	return TOptional<float>();
}

FText SGenlockProvider::GetFPSTooltip() const
{
	if (bIsAvgIdleTimeValid)
	{
		if (UFixedFrameRateCustomTimeStep* FixedTimeStep = GenlockProviderUtils::GetFixedCustomTimeStep())
		{
			const double DeciamlFixeFrameRate = FixedTimeStep->GetFixedFrameRate().AsInterval();

			return FText::Format(LOCTEXT("GetFPSTooltip", "Idle time of {0}ms out of {1}ms.")
				, FText::AsNumber(AvgIdleTime)
				, FText::AsNumber(DeciamlFixeFrameRate)
			);
		}
	}
	return FText::GetEmpty();
}

EVisibility SGenlockProvider::HandleDesiredFPSVisibility() const
{
	return GenlockProviderUtils::GetFixedCustomTimeStep() ? EVisibility::Visible : EVisibility::Collapsed;
}

namespace GenlockProviderUtils
{
	UEngineCustomTimeStep* GetCustomTimeStep()
	{
		return GEngine? GEngine->GetCustomTimeStep() : nullptr;
	}

	
	UFixedFrameRateCustomTimeStep* GetFixedCustomTimeStep() 
	{ 
		return Cast<UFixedFrameRateCustomTimeStep>(GetCustomTimeStep());
	}
}

#undef LOCTEXT_NAMESPACE
