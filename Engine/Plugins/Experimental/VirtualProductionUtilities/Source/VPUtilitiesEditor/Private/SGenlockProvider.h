// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "TickableEditorObject.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

/** 
 * This widget display the current state of a genlock setup witch is also know as the Custom TimeStep
 */
class SGenlockProvider : public SCompoundWidget, public FTickableEditorObject
{

public:
	SLATE_BEGIN_ARGS(SGenlockProvider)
	{
	}
		
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 * @param InArgs The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	//~ Begin FTickableEditorObject Interface
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;
	//~ End FTickableEditorObject Interface

private:

	double AvgIdleTime;
	bool bIsAvgIdleTimeValid;

	TSharedRef<SWidget> ConstructDesiredFPS() const;
	TSharedRef<SWidget> ConstructStateDisplay() const;

	FText HandleStateText() const;
	FSlateColor HandleStateColorAndOpacity() const;
	FText HandleDesiredFPSText() const;
	FText HandleGenlockSourceText() const;
	TOptional<float> GetFPSFraction() const;
	EVisibility HandleDesiredFPSVisibility() const;
	FText GetFPSTooltip() const;
};
