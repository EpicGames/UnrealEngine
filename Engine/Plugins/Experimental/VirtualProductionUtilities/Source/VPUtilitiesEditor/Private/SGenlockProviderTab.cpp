// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGenlockProviderTab.h"

#include "Styling/AppStyle.h"
#include "Engine/Engine.h"
#include "Engine/EngineCustomTimeStep.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/SToolBarComboButtonBlock.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "SGenlockProvider.h"
#include "VPUtilitiesEditorStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SComboButton.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"


#define LOCTEXT_NAMESPACE "GenlockProviderTab"


namespace GenlockProviderTab
{
	static FDelegateHandle LevelEditorTabManagerChangedHandle;
	static const FName NAME_GenlockProviderTab = FName("GenlockProviderTab");

	TSharedRef<SDockTab> CreateTab(const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SNew(SGenlockProviderTab)
			];
	}
}

void SGenlockProviderTab::RegisterNomadTabSpawner(TSharedRef<FWorkspaceItem> InWorkspaceItem)
{
	auto RegisterTabSpawner = [InWorkspaceItem]()
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

		LevelEditorTabManager->RegisterTabSpawner(GenlockProviderTab::NAME_GenlockProviderTab, FOnSpawnTab::CreateStatic(&GenlockProviderTab::CreateTab))
			.SetDisplayName(NSLOCTEXT("GenlockProviderTab", "DisplayName", "Genlock"))
			.SetTooltipText(NSLOCTEXT("GenlockProviderTab", "TooltipText", "Displays the current Custom Time Step."))
			.SetGroup(InWorkspaceItem)
			.SetIcon(FSlateIcon(FVPUtilitiesEditorStyle::GetStyleSetName(), "TabIcons.Genlock.Small"));
	};

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	if (LevelEditorModule.GetLevelEditorTabManager())
	{
		RegisterTabSpawner();
	}
	else
	{
		GenlockProviderTab::LevelEditorTabManagerChangedHandle = LevelEditorModule.OnTabManagerChanged().AddLambda(RegisterTabSpawner);
	}
}

void SGenlockProviderTab::UnregisterNomadTabSpawner()
{
	if (FSlateApplication::IsInitialized() && FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor");
		TSharedPtr<FTabManager> LevelEditorTabManager;
		if (LevelEditorModule)
		{
			LevelEditorTabManager = LevelEditorModule->GetLevelEditorTabManager();
			LevelEditorModule->OnTabManagerChanged().Remove(GenlockProviderTab::LevelEditorTabManagerChangedHandle);
		}

		if (LevelEditorTabManager.IsValid())
		{
			LevelEditorTabManager->UnregisterTabSpawner(GenlockProviderTab::NAME_GenlockProviderTab);
		}
	}
}

void SGenlockProviderTab::Construct(const FArguments& InArgs)
{
	TSharedRef< SWidget > ButtonContent = SNew(SComboButton)
		.ContentPadding(0)
		.ButtonStyle(&FCoreStyle::Get(), "ToolBar.Button")
		.ForegroundColor(FCoreStyle::Get().GetSlateColor("DefaultForeground"))
		.ButtonContent()
		[
			SNullWidget::NullWidget
		]
		.OnGetMenuContent(this, &SGenlockProviderTab::OnGetMenuContent);

	ButtonContent->SetEnabled(MakeAttributeLambda([] { return (GEngine && GEngine->GetCustomTimeStep() != nullptr); }));

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.f)
					[
						SNew(SGenlockProvider)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2, 0, 2, 0)
					[
						ButtonContent
					]
				]
			]
		]
	];
}

TSharedRef<SWidget> SGenlockProviderTab::OnGetMenuContent()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	check(GEngine);
	if (GEngine->GetCustomTimeStep())
	{
		MenuBuilder.BeginSection(TEXT("CustomTimeStep"), LOCTEXT("CustomTimeStep", "Custom Time Step"));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ReapplyMenuLabel", "Reinitialize"),
			LOCTEXT("ReapplyMenuToolTip", "Reinitialize the current Custom Time Step."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateUObject(GEngine, &UEngine::ReinitializeCustomTimeStep))
		);

		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
