// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FWorkspaceItem;
class SWidget;

/*
 * SGenlockProviderTab
 */
class SGenlockProviderTab : public SCompoundWidget
{
public:
	static void RegisterNomadTabSpawner(TSharedRef<FWorkspaceItem> InWorkspaceItem);
	static void UnregisterNomadTabSpawner();

public:
	SLATE_BEGIN_ARGS(SGenlockProviderTab) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	TSharedRef<SWidget> OnGetMenuContent();
};
