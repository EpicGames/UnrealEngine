// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimecodeProvider.h"

#include "EditorFontGlyphs.h"
#include "Styling/AppStyle.h"
#include "Engine/Engine.h"
#include "Misc/App.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/STimecode.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"

void STimecodeProvider::Construct(const FArguments& InArgs)
{
	OverrideTimecodeProvider = InArgs._OverrideTimecodeProvider;

	TSharedRef<SWidget> StateWidget = InArgs._DisplaySynchronizationState ? SNew(STextBlock)
		.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
		.Text(this, &STimecodeProvider::HandleStateText)
		.ColorAndOpacity(this, &STimecodeProvider::HandleIconColorAndOpacity)
		: SNullWidget::NullWidget;

	TSharedRef<SWidget> FrameRateWidget = InArgs._DisplayFrameRate ? SNew(STextBlock)
		.Text(MakeAttributeLambda([this]
		{
			if (const UTimecodeProvider* TimecodeProviderPtr = GetTimecodeProvider())
			{
				return TimecodeProviderPtr->GetFrameRate().ToPrettyText();
			}
			return NSLOCTEXT("TimecodeProvider", "Undefined", "<Undefined>");
		}))
		.Font(InArgs._TimecodeProviderFont)
		.ColorAndOpacity(InArgs._TimecodeProviderColor)
		: SNullWidget::NullWidget;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, 4.f, 0.f)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				StateWidget
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(MakeAttributeLambda([this]
				{
					if (const UTimecodeProvider* TimecodeProviderPtr = GetTimecodeProvider())
					{
						return FText::FromName(TimecodeProviderPtr->GetFName());
					}
					return NSLOCTEXT("TimecodeProvider", "Undefined", "<Undefined>");
				}))
				.Font(InArgs._TimecodeProviderFont)
				.ColorAndOpacity(InArgs._TimecodeProviderColor)
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.FillWidth(1.f)
			[
				FrameRateWidget
			]
		]
		+ SVerticalBox::Slot()
		.Padding(0, -4, 0, 0)
		.AutoHeight()
		[
			SNew(STimecode)
			.Timecode(MakeAttributeLambda([this]
				{
					if (const UTimecodeProvider* OverrideTimecodeProviderPtr = OverrideTimecodeProvider.Get().Get())
					{
						return OverrideTimecodeProviderPtr->GetDelayedTimecode();
					}

					// If we use the Engine's TimecodeProvider, get the timecode for the current frame
					return FApp::GetTimecode();
				}))
			.DisplayLabel(InArgs._DisplayLabel)
			.TimecodeFont(InArgs._TimecodeFont)
			.TimecodeColor(InArgs._TimecodeColor)
		]
	];
}

const UTimecodeProvider* STimecodeProvider::GetTimecodeProvider() const
{
	const UTimecodeProvider* TimecodeProviderPtr = OverrideTimecodeProvider.Get().Get();
	if (!TimecodeProviderPtr)
	{
		TimecodeProviderPtr = GEngine->GetTimecodeProvider();
	}

	return TimecodeProviderPtr;
}

FSlateColor STimecodeProvider::HandleIconColorAndOpacity() const
{
	FSlateColor Result = FSlateColor::UseForeground();
	if (const UTimecodeProvider* TimecodeProviderPtr = GetTimecodeProvider())
	{
		ETimecodeProviderSynchronizationState State = TimecodeProviderPtr->GetSynchronizationState();
		switch (State)
		{
		case ETimecodeProviderSynchronizationState::Closed:
		case ETimecodeProviderSynchronizationState::Error:
			return FLinearColor::Red;
		case ETimecodeProviderSynchronizationState::Synchronized:
			return FLinearColor::Green;
		case ETimecodeProviderSynchronizationState::Synchronizing:
			return FLinearColor::Yellow;
		}
	}
	return FSlateColor::UseForeground();
}

FText STimecodeProvider::HandleStateText() const
{
	if (const UTimecodeProvider* TimecodeProviderPtr = GetTimecodeProvider())
	{
		ETimecodeProviderSynchronizationState State = TimecodeProviderPtr->GetSynchronizationState();
		switch (State)
		{
		case ETimecodeProviderSynchronizationState::Error:
		case ETimecodeProviderSynchronizationState::Closed:
			return FEditorFontGlyphs::Ban;
		case ETimecodeProviderSynchronizationState::Synchronized:
			return FEditorFontGlyphs::Film;
		case ETimecodeProviderSynchronizationState::Synchronizing:
			return FEditorFontGlyphs::Hourglass_O;
		}
	}

	return FEditorFontGlyphs::Exclamation;
}
