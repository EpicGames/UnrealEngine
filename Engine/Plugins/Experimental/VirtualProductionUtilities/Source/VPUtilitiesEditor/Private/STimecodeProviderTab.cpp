// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimecodeProviderTab.h"

#include "Styling/AppStyle.h"
#include "Engine/Engine.h"
#include "Engine/TimecodeProvider.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/SToolBarComboButtonBlock.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "ObjectEditorUtils.h"
#include "ScopedTransaction.h"
#include "STimecodeProvider.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

LLM_DEFINE_TAG(VirtualProductionUtilities_TimecodeProviderTab);
#define LOCTEXT_NAMESPACE "TimecodeProviderTab"

namespace TimecodeProviderTab
{
	static FDelegateHandle LevelEditorTabManagerChangedHandle;
	static const FName NAME_TimecodeProviderTab = FName("TimecodeProviderTab");

	TSharedRef<SDockTab> CreateTab(const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SNew(STimecodeProviderTab)
			];
	}
}

void STimecodeProviderTab::RegisterNomadTabSpawner()
{
	LLM_SCOPE_BYTAG(VirtualProductionUtilities_TimecodeProviderTab);

	auto RegisterTabSpawner = []()
	{
		LLM_SCOPE_BYTAG(VirtualProductionUtilities_TimecodeProviderTab);
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

		LevelEditorTabManager->RegisterTabSpawner(TimecodeProviderTab::NAME_TimecodeProviderTab, FOnSpawnTab::CreateStatic(&TimecodeProviderTab::CreateTab))
			.SetDisplayName(NSLOCTEXT("TimecodeProviderTab", "DisplayName", "Timecode Provider"))
			.SetTooltipText(NSLOCTEXT("TimecodeProviderTab", "TooltipText", "Displays the Timecode and the state of the current Timecode Provider."))
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorVirtualProductionCategory())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "TimecodeProvider.TabIcon"));
	};

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	if (LevelEditorModule.GetLevelEditorTabManager())
	{
		RegisterTabSpawner();
	}
	else
	{
		TimecodeProviderTab::LevelEditorTabManagerChangedHandle = LevelEditorModule.OnTabManagerChanged().AddLambda(RegisterTabSpawner);
	}
}

void STimecodeProviderTab::UnregisterNomadTabSpawner()
{
	if (FSlateApplication::IsInitialized() && FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor");
		TSharedPtr<FTabManager> LevelEditorTabManager;
		if (LevelEditorModule)
		{
			LevelEditorTabManager = LevelEditorModule->GetLevelEditorTabManager();
			LevelEditorModule->OnTabManagerChanged().Remove(TimecodeProviderTab::LevelEditorTabManagerChangedHandle);
		}

		if (LevelEditorTabManager.IsValid())
		{
			LevelEditorTabManager->UnregisterTabSpawner(TimecodeProviderTab::NAME_TimecodeProviderTab);
		}
	}
}

void STimecodeProviderTab::Construct(const FArguments& InArgs)
{
	TSharedRef< SWidget > ButtonContent = SNew(SComboButton)
		.ContentPadding(0)
		.ButtonStyle(&FCoreStyle::Get(), "ToolBar.Button")
		.ForegroundColor(FCoreStyle::Get().GetSlateColor("DefaultForeground"))
		.ButtonContent()
		[
			SNullWidget::NullWidget
		]
		.OnGetMenuContent(this, &STimecodeProviderTab::OnGetMenuContent);

	ButtonContent->SetEnabled(MakeAttributeLambda([] { return GEngine && GEngine->GetTimecodeProvider() != nullptr; }));

	ChildSlot
	[
		SNew(SBorder)
		.Padding(FMargin(0, 3, 0, 0))
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(4, -4, 2, 0)
					[
						SNew(STimecodeProvider)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2, 0, 2, 0)
					[
						ButtonContent
					]
				]
			]
		]
	];
}

TSharedRef<SWidget> STimecodeProviderTab::OnGetMenuContent()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	check(GEngine);
	if (GEngine->GetTimecodeProvider())
	{
		MenuBuilder.BeginSection(TEXT("TimecodeProvider"), LOCTEXT("TimecodeProvider", "Timecode Provider"));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ReapplyMenuLabel", "Reinitialize"),
			LOCTEXT("ReapplyMenuToolTip", "Reinitialize the current Timecode Provider."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateUObject(GEngine, &UEngine::ReinitializeTimecodeProvider))
			);

		MenuBuilder.EndSection();
	}

	if (GEngine->GetTimecodeProvider())
	{
		MenuBuilder.BeginSection("Settings", LOCTEXT("Settings", "Settings"));
		{
			TSharedRef<SWidget> RefreshDelay = SNew(SSpinBox<float>)
				.ToolTipText(LOCTEXT("FrameDelay_ToolTip", "Number of frames to subtract from the original timecode."))
				.Value(this, &STimecodeProviderTab::GetFrameDelay)
				.MinValue(TOptional<float>())
				.MaxValue(TOptional<float>())
				.OnValueCommitted(this, &STimecodeProviderTab::SetFrameDelay);

			MenuBuilder.AddWidget(RefreshDelay, LOCTEXT("FrameDelay", "Frame Delay"));
		}
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

float STimecodeProviderTab::GetFrameDelay() const
{
	if (UTimecodeProvider* TimecodeProviderPtr = GEngine->GetTimecodeProvider())
	{
		return TimecodeProviderPtr->FrameDelay;
	}
	return 0.f;
}

void STimecodeProviderTab::SetFrameDelay(float InNewValue, ETextCommit::Type)
{
	if (UTimecodeProvider* TimecodeProviderPtr = GEngine->GetTimecodeProvider())
	{
		if (TimecodeProviderPtr->FrameDelay != InNewValue)
		{
			FObjectEditorUtils::SetPropertyValue(TimecodeProviderPtr, GET_MEMBER_NAME_CHECKED(UTimecodeProvider, FrameDelay), InNewValue);
		}
	}
}

#undef LOCTEXT_NAMESPACE
