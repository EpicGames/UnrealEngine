// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/FrameRate.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SWidget;

/*
 * STimecodeProviderTab
 */
class STimecodeProviderTab : public SCompoundWidget
{
public:
	static void RegisterNomadTabSpawner();
	static void UnregisterNomadTabSpawner();

public:
	SLATE_BEGIN_ARGS(STimecodeProviderTab) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	float GetFrameDelay() const;
	void SetFrameDelay(float, ETextCommit::Type);

private:
	TSharedRef<SWidget> OnGetMenuContent();
};
