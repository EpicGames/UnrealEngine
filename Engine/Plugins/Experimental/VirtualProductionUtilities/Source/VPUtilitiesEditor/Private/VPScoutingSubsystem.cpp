// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPScoutingSubsystem.h"
#include "VPUtilitiesEditorModule.h"

#include "IVREditorModule.h"
#include "VREditorMode.h"
#include "UI/VREditorUISystem.h"
#include "EditorUtilityWidget.h"
#include "Engine/AssetManager.h"
#include "LevelEditorActions.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/Script.h"
#include "UObject/UObjectGlobals.h"
#include "VPSettings.h"
#include "VPUtilitiesEditorSettings.h"

LLM_DEFINE_TAG(VirtualProductionUtilities_VPScoutingSubsystem);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
/* UVPScoutingSubsystem name
 *****************************************************************************/
const FName UVPScoutingSubsystem::VProdPanelID = FName(TEXT("VirtualProductionPanel"));
const FName UVPScoutingSubsystem::VProdPanelLeftID = FName(TEXT("VirtualProductionPanelLeft"));
const FName UVPScoutingSubsystem::VProdPanelRightID = FName(TEXT("VirtualProductionPanelRight"));
const FName UVPScoutingSubsystem::VProdPanelContextID = FName(TEXT("VirtualProductionPanelContext"));
const FName UVPScoutingSubsystem::VProdPanelTimelineID = FName(TEXT("VirtualProductionPanelTimeline"));
const FName UVPScoutingSubsystem::VProdPanelMeasureID = FName(TEXT("VirtualProductionPanelMeasure"));
const FName UVPScoutingSubsystem::VProdPanelGafferID = FName(TEXT("VirtualProductionPanelGaffer"));


/* UVPScoutingSubsystemGestureManagerBase
 *****************************************************************************/
UVPScoutingSubsystemGestureManagerBase::UVPScoutingSubsystemGestureManagerBase()
{
	LLM_SCOPE_BYTAG(VirtualProductionUtilities_VPScoutingSubsystem);

	if (!HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
	{
		IVREditorModule::Get().OnVREditingModeEnter().AddUObject(this, &UVPScoutingSubsystemGestureManagerBase::OnVREditingModeEnterCallback);
		IVREditorModule::Get().OnVREditingModeExit().AddUObject(this, &UVPScoutingSubsystemGestureManagerBase::OnVREditingModeExitCallback);
	}
}

void UVPScoutingSubsystemGestureManagerBase::BeginDestroy()
{
	LLM_SCOPE_BYTAG(VirtualProductionUtilities_VPScoutingSubsystem);

	if (!HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
	{
		IVREditorModule::Get().OnVREditingModeEnter().RemoveAll(this);
		IVREditorModule::Get().OnVREditingModeExit().RemoveAll(this);
	}

	Super::BeginDestroy();
}

void UVPScoutingSubsystemGestureManagerBase::Tick(float DeltaTime)
{
	FEditorScriptExecutionGuard ScriptGuard;
	EditorTick(DeltaTime);
}

bool UVPScoutingSubsystemGestureManagerBase::IsTickable() const
{
	if (IVREditorModule::IsAvailable())
	{
		return IVREditorModule::Get().IsVREditorModeActive();
	}
	return false;
}

TStatId UVPScoutingSubsystemGestureManagerBase::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UVPScoutingSubsystemGestureManagerBase, STATGROUP_Tickables);
}

void UVPScoutingSubsystemGestureManagerBase::OnVREditingModeEnterCallback()
{
	FEditorScriptExecutionGuard ScriptGuard;
	OnVREditingModeEnter();
}

void UVPScoutingSubsystemGestureManagerBase::OnVREditingModeExitCallback()
{
	FEditorScriptExecutionGuard ScriptGuard;
	OnVREditingModeExit();
}

void UVPScoutingSubsystemGestureManagerBase::EditorTick_Implementation(float DeltaSeconds)
{

}

void UVPScoutingSubsystemGestureManagerBase::OnVREditingModeEnter_Implementation()
{

}

void UVPScoutingSubsystemGestureManagerBase::OnVREditingModeExit_Implementation()
{

}

/* UVPScoutingSubsystem
 *****************************************************************************/
UVPScoutingSubsystem::UVPScoutingSubsystem()
	: UEditorSubsystem(),
	GripNavSpeedCoeff(4.0f)
{
}

void UVPScoutingSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	UE_LOGF(LogVPUtilitiesEditor, Log, "VP Scouting subsystem initialized.");

	// Load the ScoutingHelper implemented in BP. See BaseVirtualProductionUtilitites.ini
	VPSubsystemHelpers = nullptr;
	FSoftClassPath ClassPath = GetDefault<UVPUtilitiesEditorSettings>()->ScoutingSubsystemEditorUtilityClassPath;
	ClassPath.LoadAsync(FLoadSoftObjectPathAsyncDelegate::CreateWeakLambda(this, [this](const FSoftObjectPath&, UObject* LoadedObject)
	{
		if (UClass* EditorUtilityClass = Cast<UClass>(LoadedObject))
		{
			VPSubsystemHelpers = NewObject<UVPScoutingSubsystemHelpersBase>(GetTransientPackage(), EditorUtilityClass);
		}
	}));
	
	// to do final initializations at the right time
	EngineInitCompleteDelegate = FCoreDelegates::OnFEngineLoopInitComplete.AddUObject(this, &UVPScoutingSubsystem::OnEngineInitComplete);
}

void UVPScoutingSubsystem::Deinitialize()
{
	VPSubsystemHelpers = nullptr;
}

void UVPScoutingSubsystem::OnEngineInitComplete()
{
	LLM_SCOPE_BYTAG(VirtualProductionUtilities_VPScoutingSubsystem);

	FCoreDelegates::OnFEngineLoopInitComplete.Remove(EngineInitCompleteDelegate);
	EngineInitCompleteDelegate.Reset();

	// Load the GestureManager implemented in BP. See BaseVirtualProductionUtilitites.ini
	// GestureManager needs the Take module, load it once the engine is loaded.
	GestureManager = nullptr;
	if (UClass* EditorUtilityClass = GetDefault<UVPUtilitiesEditorSettings>()->GestureManagerEditorUtilityClassPath.TryLoadClass<UVPScoutingSubsystemGestureManagerBase>())
	{
		GestureManager = NewObject<UVPScoutingSubsystemGestureManagerBase>(GetTransientPackage(), EditorUtilityClass);
	}

	// In debug some asset take a long time to load and crash the engine, preload those asset in async mode to prevent that
	for (const FSoftClassPath& ClassAssetPath : GetDefault<UVPUtilitiesEditorSettings>()->AdditionnalClassToLoad)
	{
		FStreamableManager& StreamableManager = UAssetManager::Get().GetStreamableManager();
		StreamableManager.RequestAsyncLoad(ClassAssetPath);
	}
}

void UVPScoutingSubsystem::ToggleVRScoutingUI(FVREditorFloatingUICreationContext& CreationContext)
{	
	// @todo: Add lookup like like bool UVREditorUISystem::EditorUIPanelExists(const VREditorPanelID& InPanelID) const
	// Return if users try to create a panel that already exists
		
	if (CreationContext.WidgetClass == nullptr || CreationContext.PanelID == TEXT(""))
	{
		UE_LOGF(LogVPUtilitiesEditor, Warning, "UVPScoutingSubsystem::ToggleVRScoutingUI - WidgetClass or PanelID can't be null.");
		return; // @todo: Remove early rejection code, hook up UVPSettings::VirtualScoutingUI instead
	}
	
	// Account for actors trying to call this function from their destructor when VR mode ends (UI system is one of the earliest systems getting shut down)	
	UVREditorMode const* const VRMode = IVREditorModule::Get().GetVRMode();
	if (VRMode && VRMode->UISystemIsActive())
	{
		bool bPanelVisible = VRMode->GetUISystem().IsShowingEditorUIPanel(CreationContext.PanelID);

		// Close panel if currently visible
		if (bPanelVisible)
		{
			// Close the existing panel by passing null as the widget. We don't care about any of the other parameters in this case
			CreationContext.WidgetClass = nullptr;
			CreationContext.PanelSize = FVector2D(1, 1); // Guard against 0,0 user input. The actual size is not important when closing a panel, but a check() would trigger
			IVREditorModule::Get().UpdateExternalUMGUI(CreationContext);
		}
		else // Otherwise open a new one - with the user-defined VProd UI being the default
		{
			// @todo: Currently won't ever be true
			if (CreationContext.WidgetClass == nullptr)
			{
				const TSoftClassPtr<UEditorUtilityWidget> WidgetClass = GetDefault<UVPUtilitiesEditorSettings>()->VirtualScoutingUI;
				WidgetClass.LoadSynchronous();
				if (WidgetClass.IsValid())
				{
					CreationContext.WidgetClass = WidgetClass.Get();
				}
			}

			if (CreationContext.WidgetClass != nullptr)
			{
				IVREditorModule::Get().UpdateExternalUMGUI(CreationContext);
			}
			else
			{
				UE_LOGF(LogVPUtilitiesEditor, Warning, "UVPScoutingSubsystem::ToggleVRScoutingUI - Failed to open widget-based VR window.");
			}
		}
	}
}

void UVPScoutingSubsystem::HideInfoDisplayPanel()
{
	UVREditorMode*  VRMode = IVREditorModule::Get().GetVRMode();
	if (VRMode && VRMode->UISystemIsActive())
	{
		UVREditorUISystem& UISystem = VRMode->GetUISystem();
		AVREditorFloatingUI* Panel = UISystem.GetPanel(UVREditorUISystem::InfoDisplayPanelID);
		if (Panel->IsUIVisible()) 
		{
			Panel->ShowUI(false);
		}
	}
}

bool UVPScoutingSubsystem::IsVRScoutingUIOpen(const FName& PanelID)
{
	return IVREditorModule::Get().GetVRMode()->GetUISystem().IsShowingEditorUIPanel(PanelID);
}

AVREditorFloatingUI * UVPScoutingSubsystem::GetPanelActor(const FName& PanelID) const
{
	return IVREditorModule::Get().GetVRMode()->GetUISystem().GetPanel(PanelID);
	
}

UUserWidget * UVPScoutingSubsystem::GetPanelWidget(const FName & PanelID) const
{
	AVREditorFloatingUI* Panel = GetPanelActor(PanelID);
	if (Panel == nullptr)
	{
		return nullptr;
	}
	else
	{
		return Panel->GetUserWidget();
	}
}

TArray<UVREditorInteractor*> UVPScoutingSubsystem::GetActiveEditorVRControllers()
{
	IVREditorModule& VREditorModule = IVREditorModule::Get();
	UVREditorMode* VRMode = VREditorModule.GetVRMode();
	
	const TArray<UVREditorInteractor*> Interactors = VRMode->GetVRInteractors();
	ensureMsgf(Interactors.Num() == 2, TEXT("Expected 2 VR controllers from VREditorMode, got %d"), Interactors.Num());
	return Interactors;		
}


FString UVPScoutingSubsystem::GetDirectorName()
{
	FString DirectorName = GetDefault<UVPSettings>()->DirectorName;
	if (DirectorName == TEXT(""))
	{
		DirectorName = "Undefined";
	}
	return DirectorName;
}

FString UVPScoutingSubsystem::GetShowName()
{
	FString ShowName = GetDefault<UVPSettings>()->ShowName;
	if (ShowName == TEXT(""))
	{
		ShowName = "Undefined";
	}
	return ShowName;
}

bool UVPScoutingSubsystem::IsUsingMetricSystem()
{
	return GetDefault<UVPUtilitiesEditorSettings>()->bUseMetric;
}

void UVPScoutingSubsystem::SetIsUsingMetricSystem(const bool bInUseMetricSystem)
{
	UVPUtilitiesEditorSettings* VPUtilitiesEditorSettings = GetMutableDefault<UVPUtilitiesEditorSettings>();
	VPUtilitiesEditorSettings->bUseMetric = bInUseMetricSystem;
	VPUtilitiesEditorSettings->SaveConfig();
}

bool UVPScoutingSubsystem::IsUsingTransformGizmo()
{
	return GetDefault<UVPUtilitiesEditorSettings>()->bUseTransformGizmo;
}

void UVPScoutingSubsystem::SetIsUsingTransformGizmo(const bool bInIsUsingTransformGizmo)
{
	UVPUtilitiesEditorSettings* VPUtilitiesEditorSettings = GetMutableDefault<UVPUtilitiesEditorSettings>();
	if (bInIsUsingTransformGizmo != VPUtilitiesEditorSettings->bUseTransformGizmo)
	{
		VPUtilitiesEditorSettings->bUseTransformGizmo = bInIsUsingTransformGizmo;
		SetShowTransformGizmoCVar(bInIsUsingTransformGizmo);
		VPUtilitiesEditorSettings->SaveConfig();
	}
}

void UVPScoutingSubsystem::SetShowTransformGizmoCVar(const bool bInShowTransformGizmoCVar)
{
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("VI.ShowTransformGizmo"));
	CVar->Set(bInShowTransformGizmoCVar);
}

float UVPScoutingSubsystem::GetFlightSpeed()
{
	return GetDefault<UVPUtilitiesEditorSettings>()->FlightSpeed;
}

void UVPScoutingSubsystem::SetFlightSpeed(const float InFlightSpeed)
{
	UVPUtilitiesEditorSettings* VPUtilitiesEditorSettings = GetMutableDefault<UVPUtilitiesEditorSettings>();
	VPUtilitiesEditorSettings->FlightSpeed = InFlightSpeed;
	VPUtilitiesEditorSettings->SaveConfig();
}

float UVPScoutingSubsystem::GetGripNavSpeed()
{
	return GetDefault<UVPUtilitiesEditorSettings>()->GripNavSpeed;
}

void UVPScoutingSubsystem::SetGripNavSpeed(const float InGripNavSpeed)
{
	UVPUtilitiesEditorSettings* VPUtilitiesEditorSettings = GetMutableDefault<UVPUtilitiesEditorSettings>();
	VPUtilitiesEditorSettings->GripNavSpeed = InGripNavSpeed;
	VPUtilitiesEditorSettings->SaveConfig();
}

bool UVPScoutingSubsystem::IsUsingInertiaDamping()
{
	return GetDefault<UVPUtilitiesEditorSettings>()->bUseGripInertiaDamping;
}

void UVPScoutingSubsystem::SetIsUsingInertiaDamping(const bool bInIsUsingInertiaDamping)
{
	//Save this value in editor settings and set the console variable which is used for inertia damping
	UVPUtilitiesEditorSettings* VPUtilitiesEditorSettings = GetMutableDefault<UVPUtilitiesEditorSettings>();
	VPUtilitiesEditorSettings->bUseGripInertiaDamping = bInIsUsingInertiaDamping;
	
	if (bInIsUsingInertiaDamping)
	{
		SetInertiaDampingCVar(VPUtilitiesEditorSettings->InertiaDamping);
	}
	else
	{
		SetInertiaDampingCVar(0);
	}
	VPUtilitiesEditorSettings->SaveConfig();
}

void UVPScoutingSubsystem::SetInertiaDampingCVar(const float InInertiaDamping)
{
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("VI.HighSpeedInertiaDamping"));
	CVar->Set(InInertiaDamping);
}

bool UVPScoutingSubsystem::IsHelperSystemEnabled()
{
	return GetDefault<UVPUtilitiesEditorSettings>()->bIsHelperSystemEnabled;
}

void UVPScoutingSubsystem::SetIsHelperSystemEnabled(const bool bInIsHelperSystemEnabled)
{
	UVPUtilitiesEditorSettings* VPUtilitiesEditorSettings = GetMutableDefault<UVPUtilitiesEditorSettings>();
	VPUtilitiesEditorSettings->bIsHelperSystemEnabled = bInIsHelperSystemEnabled;
	VPUtilitiesEditorSettings->SaveConfig();
}

UVREditorMode* UVPScoutingSubsystem::GetVREditorMode()
{
	return IVREditorModule::Get().GetVRMode();
}

bool UVPScoutingSubsystem::EnterVRMode()
{
	IVREditorModule& VREditorModule = IVREditorModule::Get();
	if (VREditorModule.IsVREditorAvailable())
	{
		VREditorModule.EnableVREditor(true);
		return true;
	}

	return false;
}

void UVPScoutingSubsystem::ExitVRMode()
{
	IVREditorModule& VREditorModule = IVREditorModule::Get();
	if (VREditorModule.IsVREditorEnabled())
	{
		VREditorModule.EnableVREditor(false);
	}
}

bool UVPScoutingSubsystem::IsLocationGridSnappingEnabled()
{
	return FLevelEditorActionCallbacks::LocationGridSnap_IsChecked();
}

void UVPScoutingSubsystem::ToggleLocationGridSnapping()
{
	FLevelEditorActionCallbacks::LocationGridSnap_Clicked();
}

bool UVPScoutingSubsystem::IsRotationGridSnappingEnabled()
{
	return FLevelEditorActionCallbacks::RotationGridSnap_IsChecked();
}

void UVPScoutingSubsystem::ToggleRotationGridSnapping()
{
	FLevelEditorActionCallbacks::RotationGridSnap_Clicked();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS