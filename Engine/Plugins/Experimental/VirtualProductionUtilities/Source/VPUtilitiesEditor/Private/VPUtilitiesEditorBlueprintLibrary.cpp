// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPUtilitiesEditorBlueprintLibrary.h"
#include "VPScoutingSubsystem.h"
#include "VPUtilitiesEditorModule.h"
#include "Editor.h"

#include "Modules/ModuleManager.h"
#include "Misc/FileHelper.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "Factories/TextureFactory.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "EditorFramework/AssetImportData.h"
#include "FileHelpers.h"
#include "Engine/Texture2D.h"
#include "OSCServer.h"

AVPEditorTickableActorBase* UVPUtilitiesEditorBlueprintLibrary::SpawnVPEditorTickableActor(UObject* ContextObject, const TSubclassOf<AVPEditorTickableActorBase> ActorClass, const FVector Location, const FRotator Rotation)
{
	if (ActorClass.Get() == nullptr)
	{
		UE_LOGF(LogVPUtilitiesEditor, Warning, "VPUtilitiesEditorBlueprintLibrary::SpawnVPEditorTickableActor - The ActorClass is invalid");
		return nullptr;
	}

	UWorld* World = ContextObject ? ContextObject->GetWorld() : nullptr;
	if (World == nullptr)
	{
		UE_LOGF(LogVPUtilitiesEditor, Warning, "VPUtilitiesEditorBlueprintLibrary::SpawnVPEditorTickableActor - The ContextObject is invalid.");
		return nullptr;
	}

	FActorSpawnParameters SpawnParams;
	SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	AVPEditorTickableActorBase* NewActor = World->SpawnActor<AVPEditorTickableActorBase>(ActorClass.Get(), Location, Rotation, SpawnParams);
	return NewActor;
}

AVPTransientEditorTickableActorBase* UVPUtilitiesEditorBlueprintLibrary::SpawnVPTransientEditorTickableActor(UObject* ContextObject, const TSubclassOf<AVPTransientEditorTickableActorBase> ActorClass, const FVector Location, const FRotator Rotation)
{
	if (ActorClass.Get() == nullptr)
	{
		UE_LOGF(LogVPUtilitiesEditor, Warning, "VPUtilitiesEditorBlueprintLibrary::SpawnVPTransientEditorTickableActor - The ActorClass is invalid");
		return nullptr;
	}

	UWorld* World = ContextObject ? ContextObject->GetWorld() : nullptr;
	if (World == nullptr)
	{
		UE_LOGF(LogVPUtilitiesEditor, Warning, "VPUtilitiesEditorBlueprintLibrary::SpawnVPTransientEditorTickableActor - The ContextObject is invalid.");
		return nullptr;
	}

	FActorSpawnParameters SpawnParams;
	SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	AVPTransientEditorTickableActorBase* NewActor = World->SpawnActor<AVPTransientEditorTickableActorBase>(ActorClass.Get(), Location, Rotation, SpawnParams);
	return NewActor;
}

UTexture* UVPUtilitiesEditorBlueprintLibrary::ImportSnapshotTexture(FString FileName, FString SubFolderName, FString AbsolutePathPackage)
{
	UTexture* UnrealTexture = NULL;

	if (FileName.IsEmpty())
	{
		return UnrealTexture;
	}

	if (AbsolutePathPackage.IsEmpty())
	{
		AbsolutePathPackage = FPaths::ProjectSavedDir() + "VirtualProduction/Snapshots/" + SubFolderName + "/";
	}

	FString TextureName = FPaths::GetBaseFilename(FileName);
	TextureName = ObjectTools::SanitizeObjectName(TextureName);
	FString Extension = FPaths::GetExtension(FileName).ToLower();

	FString PackageName = TEXT("/Game/Snapshots/" + SubFolderName + "/");
	PackageName += TextureName;
	UPackage* Package = CreatePackage(*PackageName);
	Package->FullyLoad();

	// try opening from absolute path
	FileName = AbsolutePathPackage + FileName;
	TArray<uint8> TextureData;
	if (!(FFileHelper::LoadFileToArray(TextureData, *FileName) && TextureData.Num() > 0))
	{
		UE_LOGF(LogVPUtilitiesEditor, Warning, "Unable to find Texture file %ls", *FileName);
	}
	else
	{
		UTextureFactory* TextureFactory = NewObject<UTextureFactory>();
		TextureFactory->AddToRoot();
		TextureFactory->SuppressImportOverwriteDialog();

		const uint8* PtrTexture = TextureData.GetData();
		UnrealTexture = Cast<UTexture>(TextureFactory->FactoryCreateBinary(UTexture2D::StaticClass(), Package, *TextureName, RF_Standalone | RF_Public, NULL, *Extension, PtrTexture, PtrTexture + TextureData.Num(), GWarn));
		if (UnrealTexture != nullptr)
		{
			UnrealTexture->AssetImportData->Update(FileName);

			Package->MarkPackageDirty();

			// Notify the asset registry
			FAssetRegistryModule::AssetCreated(UnrealTexture);
		}

		TextureFactory->RemoveFromRoot();
	}

	return UnrealTexture;
}

class UOSCServer* UVPUtilitiesEditorBlueprintLibrary::GetDefaultOSCServer()
{
	return FVPUtilitiesEditorModule::Get().GetOSCServer();
}
