// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPUtilitiesEditorModule.h"

#include "Framework/Docking/WorkspaceItem.h"
#include "GameplayTagContainer.h"
#include "Editor.h"
#include "Engine/Engine.h"
#include "EditorUtilitySubsystem.h"
#include "HAL/IConsoleManager.h"
#include "LevelEditor.h"
#include "IPlacementModeModule.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "OSCManager.h"
#include "OSCServer.h"
#include "SGenlockProviderTab.h"
#include "STimecodeProviderTab.h"
#include "Textures/SlateIcon.h"
#include "VPRolesSubsystem.h"
#include "VPSettings.h"
#include "VPUtilitiesEditorSettings.h"
#include "VPUtilitiesEditorStyle.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "LevelEditorOutlinerSettings.h"

LLM_DEFINE_TAG(VirtualProductionUtilities_VPUtilitiesEditor);
#define LOCTEXT_NAMESPACE "VPUtilitiesEditor"

DEFINE_LOG_CATEGORY(LogVPUtilitiesEditor);

const FName FVPUtilitiesEditorModule::VPRoleNotificationBarIdentifier = TEXT("VPRoles");

void FVPUtilitiesEditorModule::StartupModule()
{
	LLM_SCOPE_BYTAG(VirtualProductionUtilities_VPUtilitiesEditor);

	FVPUtilitiesEditorStyle::Register();

	SGenlockProviderTab::RegisterNomadTabSpawner(WorkspaceMenu::GetMenuStructure().GetLevelEditorVirtualProductionCategory());
	STimecodeProviderTab::RegisterNomadTabSpawner();

	RegisterSettings();

	if (GetDefault<UVPUtilitiesEditorSettings>()->bStartOSCServerAtLaunch)
	{
		InitializeOSCServer();
	}
}

void FVPUtilitiesEditorModule::ShutdownModule()
{
	LLM_SCOPE_BYTAG(VirtualProductionUtilities_VPUtilitiesEditor);

	UnregisterSettings();
	STimecodeProviderTab::UnregisterNomadTabSpawner();
	SGenlockProviderTab::UnregisterNomadTabSpawner();

	FVPUtilitiesEditorStyle::Unregister();
}

UOSCServer* FVPUtilitiesEditorModule::GetOSCServer() const
{
	return OSCServer.Get();
}

const FPlacementCategoryInfo* FVPUtilitiesEditorModule::GetVirtualProductionPlacementCategoryInfo() const
{
	if (GEditor)
	{
		IPlacementModeModule& PlacmentModeModule = IPlacementModeModule::Get();

		if (const FPlacementCategoryInfo* RegisteredInfo = PlacmentModeModule.GetRegisteredPlacementCategory(FLevelEditorOutlinerBuiltInCategories::VirtualProduction()))
		{
			return RegisteredInfo;
		}
		else
		{
			FPlacementCategoryInfo Info(
				LOCTEXT("VirtualProductionCategoryName", "Virtual Production"),
				FSlateIcon(FVPUtilitiesEditorStyle::GetStyleSetName(), "PlacementBrowser.Icons.VirtualProduction"),
				FLevelEditorOutlinerBuiltInCategories::VirtualProduction(),
				TEXT("PMVirtualProduction"),
				25
			);
			Info.ShortDisplayName = LOCTEXT("VirtualProductionShortCategoryName", "VP");
			IPlacementModeModule::Get().RegisterPlacementCategory(Info);

			// This will return nullptr if the Register above failed so we don't need to explicitly check
			// RegisterPlacementCategory's return value.
			return PlacmentModeModule.GetRegisteredPlacementCategory(FLevelEditorOutlinerBuiltInCategories::VirtualProduction());
		}
	}

	return nullptr;
}

void FVPUtilitiesEditorModule::RegisterSettings()
{
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule != nullptr)
	{
		ISettingsSectionPtr SettingsSection = SettingsModule->RegisterSettings("Project", "Plugins", "VirtualProduction",
			LOCTEXT("VirtualProductionSettingsName", "Virtual Production"),
			LOCTEXT("VirtualProductionSettingsDescription", "Configure the Virtual Production settings."),
			GetMutableDefault<UVPSettings>());

		ISettingsSectionPtr EditorSettingsSection = SettingsModule->RegisterSettings("Project", "Plugins", "VirtualProductionEditor",
			LOCTEXT("VirtualProductionEditorSettingsName", "Virtual Production Editor"),
			LOCTEXT("VirtualProductionEditorSettingsDescription", "Configure the Virtual Production Editor settings."),
			GetMutableDefault<UVPUtilitiesEditorSettings>());

		ISettingsSectionPtr EditorUserSettingsSection = SettingsModule->RegisterSettings("Editor", "Plugins", "VirtualProduction",
			LOCTEXT("VirtualProductionEditorUserSettingsName", "Virtual Production"),
			LOCTEXT("VirtualProductionEditorUserSettingsNameDescription", "Configure the Virtual Production Editor user settings"),
			GetMutableDefault<UVPUtilitiesEditorUserSettings>()
		);

		EditorUserSettingsSection->OnModified().BindRaw(this, &FVPUtilitiesEditorModule::OnSettingsModified);

		EditorSettingsSection->OnModified().BindRaw(this, &FVPUtilitiesEditorModule::OnSettingsModified);
	}

	FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor");
	if (LevelEditorModule != nullptr)
	{
		FLevelEditorModule::FTitleBarItem Item;
		Item.Label = LOCTEXT("VPRolesLabel", "VP Roles: ");
		Item.Value = MakeAttributeLambda([]() { return FText::FromString(GEngine->GetEngineSubsystem<UVirtualProductionRolesSubsystem>()->GetActiveRolesString()); });
		Item.Visibility = MakeAttributeLambda([]() { return GetMutableDefault<UVPSettings>()->bShowRoleInEditor ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed; });
		LevelEditorModule->AddTitleBarItem(VPRoleNotificationBarIdentifier, Item);
	}
}

void FVPUtilitiesEditorModule::UnregisterSettings()
{
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule != nullptr)
	{
		SettingsModule->UnregisterSettings("Editor", "Plugins", "VirtualProduction");
		SettingsModule->UnregisterSettings("Project", "Plugins", "VirtualProduction");
		SettingsModule->UnregisterSettings("Project", "Plugins", "VirtualProductionEditor");
	}

	FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor");
	if (LevelEditorModule != nullptr)
	{
		LevelEditorModule->RemoveTitleBarItem(VPRoleNotificationBarIdentifier);
	}
}

void FVPUtilitiesEditorModule::InitializeOSCServer()
{
	if (OSCServer)
	{
		OSCServer->Stop();
	}

	const UVPUtilitiesEditorSettings* Settings = GetDefault<UVPUtilitiesEditorSettings>();
	FString ServerAddress = Settings->OSCServerAddress;
	uint16 ServerPort = Settings->OSCServerPort;

	const UVPUtilitiesEditorUserSettings* UserSettings = GetDefault<UVPUtilitiesEditorUserSettings>();
	const FString& OverrideAddress = UserSettings->OverrideOSCServerAddress;
	uint16 OverridePort = UserSettings->OverrideOSCServerPort;

	if (!OverrideAddress.IsEmpty())
	{
		UE_LOGF(LogVPUtilitiesEditor, Display, "OSC Server address was overridden from %ls to %ls.", *ServerAddress, *OverrideAddress);
		ServerAddress = OverrideAddress;
	}

	if (OverridePort != 0)
	{
		UE_LOGF(LogVPUtilitiesEditor, Display, "OSC Server port was overridden from %d to %d.", ServerPort, OverridePort);
		ServerPort = OverridePort;
	}

	if (OSCServer)
	{
		OSCServer->SetAddress(ServerAddress, ServerPort);
		OSCServer->Listen();
	}
	else
	{
		OSCServer.Reset(UOSCManager::CreateOSCServer(ServerAddress, ServerPort, false, true, FString(), GetTransientPackage()));
		
#if WITH_EDITOR
		// Allow it to tick in editor, so that messages are parsed.
		// Only doing it upon creation so that the user can make it non-tickable if desired (and manage that thereafter).
		if (OSCServer)
		{
			OSCServer->SetTickInEditor(true);
		}
#endif // WITH_EDITOR
	}

	const TArray<FSoftObjectPath>& ListenerPaths = Settings->StartupOSCListeners;
	for (const FSoftObjectPath& ListenerPath : ListenerPaths)
	{
		if (ListenerPath.IsValid())
		{
			UObject* Object = GetValid(ListenerPath.TryLoad());
			if (Object && GEditor)
			{
				GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>()->TryRun(Object);
			}
		}
	}
}

bool FVPUtilitiesEditorModule::OnSettingsModified()
{
	const UVPUtilitiesEditorSettings* Settings = GetDefault<UVPUtilitiesEditorSettings>();
	if (Settings->bStartOSCServerAtLaunch)
	{
		InitializeOSCServer();
	}
	else if(OSCServer)
	{
		OSCServer->Stop();
	}
	
	return true;
}


IMPLEMENT_MODULE(FVPUtilitiesEditorModule, VPUtilitiesEditor)

#undef LOCTEXT_NAMESPACE
