// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IVPUtilitiesEditorModule.h"
#include "UObject/StrongObjectPtr.h"


DECLARE_LOG_CATEGORY_EXTERN(LogVPUtilitiesEditor, Log, Log);

class UOSCServer;

class FVPUtilitiesEditorModule : public IVPUtilitiesEditorModule
{
public:
		//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	/**
	 * Get an OSC server that can be started at the module's startup.
	 */
	virtual UOSCServer* GetOSCServer() const override;

	/**
	 * Returns the Placement Mode Info for the Virtual Production category.
	 * The category will be registered if it has not already been.
	 */
	virtual const FPlacementCategoryInfo* GetVirtualProductionPlacementCategoryInfo() const override;

private:
	/** Register VPUtilities settings. */
	void RegisterSettings();

	/** Unregister VPUtilities settings */
	void UnregisterSettings();

	/** Start an OSC server and bind a an OSC listener to it. */
	void InitializeOSCServer();

	/** Handler for when VP utilities settings are changed. */
	bool OnSettingsModified();

private:
	/** The default OSC server. */
	TStrongObjectPtr<UOSCServer> OSCServer;

	/** Virtual production role identifier for the notification bar. */
	static const FName VPRoleNotificationBarIdentifier;
	
};