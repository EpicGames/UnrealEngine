// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPUtilitiesEditorSettings.h"

UVPUtilitiesEditorSettings::UVPUtilitiesEditorSettings()
	: OSCServerAddress(TEXT("127.0.0.1"))
	, OSCServerPort(5500)
{}
