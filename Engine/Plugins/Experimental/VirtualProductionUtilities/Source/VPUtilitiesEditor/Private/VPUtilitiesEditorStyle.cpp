// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPUtilitiesEditorStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

#define RootToContentDir VPUtilitiesEditorStyle::StyleInstance->RootToContentDir

#include "Styling/SlateStyleMacros.h"

namespace VPUtilitiesEditorStyle
{
	const FName NAME_StyleName(TEXT("VPUtilitiesStyle"));

	static TUniquePtr<FSlateStyleSet> StyleInstance;
}

void FVPUtilitiesEditorStyle::Register()
{
	const FVector2D Icon16x16(16.0f, 16.0f);

	VPUtilitiesEditorStyle::StyleInstance = MakeUnique<FSlateStyleSet>(VPUtilitiesEditorStyle::NAME_StyleName);
	VPUtilitiesEditorStyle::StyleInstance->SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/VirtualProductionUtilities/Content/Editor/Icons/"));

	VPUtilitiesEditorStyle::StyleInstance->Set("TabIcons.Genlock.Small", new IMAGE_BRUSH_SVG("Genlock", Icon16x16));

	VPUtilitiesEditorStyle::StyleInstance->Set("PlacementBrowser.Icons.VirtualProduction", new IMAGE_BRUSH_SVG("VirtualProduction", Icon16x16));


	FSlateStyleRegistry::RegisterSlateStyle(*VPUtilitiesEditorStyle::StyleInstance.Get());
}

void FVPUtilitiesEditorStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*VPUtilitiesEditorStyle::StyleInstance.Get());
	VPUtilitiesEditorStyle::StyleInstance.Reset();
}

FName FVPUtilitiesEditorStyle::GetStyleSetName()
{
	return VPUtilitiesEditorStyle::NAME_StyleName;
}

const ISlateStyle& FVPUtilitiesEditorStyle::Get()
{
	check(VPUtilitiesEditorStyle::StyleInstance.IsValid());
	return *VPUtilitiesEditorStyle::StyleInstance.Get();
}

#undef RootToContentDir
