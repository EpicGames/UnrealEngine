// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

class FSlateStyleSet;
class ISlateStyle;

//////////////////////////////////////////////////////////////////////////
// FVPUtilitiesEditorStyle

class FVPUtilitiesEditorStyle
{
public:
	static void Register();
	static void Unregister();

	static FName GetStyleSetName();

	static const ISlateStyle& Get();
};