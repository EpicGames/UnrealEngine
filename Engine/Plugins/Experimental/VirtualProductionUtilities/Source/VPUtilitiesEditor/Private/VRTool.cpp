// Copyright Epic Games, Inc. All Rights Reserved.

#include "VRTool.h"
#include "GameFramework/InputSettings.h"
#include "Engine/InputDelegateBinding.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

AVRTool::AVRTool(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// We don't want VR "game view" to hide the tools.
	SetActorHiddenInGame(false);
}

void AVRTool::OnConstruction(const FTransform& Transform)
{
	Super::OnConstruction(Transform);

	CreateEditorInput();
}

void AVRTool::CreateEditorInput()
{
	if (bReceivesEditorInput && !HasAnyFlags(RF_ClassDefaultObject) && !EditorOnlyInputComponent)
	{
		EditorOnlyInputComponent = NewObject<UInputComponent>(this, UInputSettings::GetDefaultInputComponentClass(), TEXT("EdUtilActor_InputComponent0"), RF_Transient);
		UInputDelegateBinding::BindInputDelegatesWithSubojects(this, EditorOnlyInputComponent);
	}
}

void AVRTool::RemoveEditorInput()
{
	ensure(!bReceivesEditorInput);

	if (EditorOnlyInputComponent)
	{
		EditorOnlyInputComponent->DestroyComponent();
	}
	EditorOnlyInputComponent = nullptr;
}

#if WITH_EDITOR
void AVRTool::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(AVRTool, bReceivesEditorInput))
	{
		SetReceivesEditorInput(bReceivesEditorInput);
	}
}
#endif


void AVRTool::SetReceivesEditorInput(bool bInValue)
{
	bReceivesEditorInput = bInValue;
	if (bReceivesEditorInput)
	{
		CreateEditorInput();
	}
	else
	{
		RemoveEditorInput();
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS