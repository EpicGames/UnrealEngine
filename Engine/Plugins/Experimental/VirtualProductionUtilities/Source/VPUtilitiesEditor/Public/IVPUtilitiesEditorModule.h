// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class UOSCServer;
struct FPlacementCategoryInfo;

class VPUTILITIESEDITOR_API IVPUtilitiesEditorModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static IVPUtilitiesEditorModule& Get()
	{
		static const FName ModuleName = "VPUtilitiesEditor";
		return FModuleManager::LoadModuleChecked<IVPUtilitiesEditorModule>(ModuleName);
	};

	/**
	 * Get an OSC server that can be started at the module's startup.
	 */
	virtual UOSCServer* GetOSCServer() const = 0;

	/**
	 * Returns the Placement Mode Info for the Virtual Production category.
	 * The category will be registered if it has not already been.
	 */
	virtual const FPlacementCategoryInfo* GetVirtualProductionPlacementCategoryInfo() const = 0;
};