// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#include "Engine/TimecodeProvider.h"
#include "Fonts/SlateFontInfo.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Misc/Timecode.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateColor.h"
#include "UObject/WeakObjectPtrTemplates.h"

struct FSlateBrush;

class VPUTILITIESEDITOR_API STimecodeProvider : public SCompoundWidget
{	
public:
	SLATE_BEGIN_ARGS(STimecodeProvider)
		: _TimecodeProviderColor(FLinearColor::Gray)
		, _DisplayFrameRate(true)
		, _DisplaySynchronizationState(true)
		, _TimecodeColor(FLinearColor::White)
		, _DisplayLabel(false)
		, _LabelColor(FLinearColor::Gray)
	{
		FSlateFontInfo NormalFontInfo = FCoreStyle::Get().GetFontStyle(TEXT("NormalText"));
		int32 OriginalSize = NormalFontInfo.Size;

		_LabelFont = NormalFontInfo;
		NormalFontInfo.Size = OriginalSize - 4;
		_TimecodeProviderFont = NormalFontInfo;
		NormalFontInfo.Size = OriginalSize + 16;
		_TimecodeFont = NormalFontInfo;
	}
		/** The font for this TimecodeProvider text */
		SLATE_ATTRIBUTE(FSlateFontInfo, TimecodeProviderFont)
		/** The color for this TimecodeProvider text */
		SLATE_ATTRIBUTE(FSlateColor, TimecodeProviderColor)
		/** Should display the TimecodeProvider's frame rate */
		SLATE_ARGUMENT(bool, DisplayFrameRate)
		/** Should display the TimecodeProvider's synchronization state */
		SLATE_ARGUMENT(bool, DisplaySynchronizationState)
		/** Override the Timecode Provider to display */
		SLATE_ATTRIBUTE(TWeakObjectPtr<UTimecodeProvider>, OverrideTimecodeProvider)

		/** The font for the timecode text */
		SLATE_ATTRIBUTE(FSlateFontInfo, TimecodeFont)
		/** The color for the timecode text */
		SLATE_ATTRIBUTE(FSlateColor, TimecodeColor)

		/** Should display the label (hours, mins, secs, frames) */
		SLATE_ATTRIBUTE(bool, DisplayLabel)
		/** The font for this label text */
		SLATE_ATTRIBUTE(FSlateFontInfo, LabelFont)
		/** The color for this label text */
		SLATE_ATTRIBUTE(FSlateColor, LabelColor)
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs);

private:

	const UTimecodeProvider* GetTimecodeProvider() const;
	FSlateColor HandleIconColorAndOpacity() const;
	FText HandleStateText() const;


private:
	TAttribute<TWeakObjectPtr<UTimecodeProvider>> OverrideTimecodeProvider;
};
