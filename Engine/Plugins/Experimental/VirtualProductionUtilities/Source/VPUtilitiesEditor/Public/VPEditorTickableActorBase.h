// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Actors/VPViewportTickableActorBase.h"
#include "VPEditorTickableActorBase.generated.h"

/** Actor that tick in the Editor viewport with the event EditorTick. */
UCLASS(Abstract)
class VPUTILITIESEDITOR_API AVPEditorTickableActorBase : public AVPViewportTickableActorBase
{
	GENERATED_BODY()
};
