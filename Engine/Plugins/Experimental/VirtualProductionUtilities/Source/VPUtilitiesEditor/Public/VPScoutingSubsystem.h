// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "EditorUtilityWidget.h"
#include "Templates/SubclassOf.h"
#include "TickableEditorObject.h"
#include "UI/VREditorFloatingUI.h"
#include "UnrealEdMisc.h"
#include "VPScoutingSubsystem.generated.h"


UENUM(BlueprintType)
enum class EVProdPanelIDs : uint8
{
	Main,
	Left,
	Right,
	Context,
	Timeline,
	Measure,
	Gaffer
};


/*
 * Base class of the helper class defined in BP
 */

UCLASS(Abstract, Blueprintable, MinimalAPI, meta = (ShowWorldContextPin))
class UE_DEPRECATED(5.5,"Code will be removed from UE5.7") UVPScoutingSubsystemHelpersBase : public UObject
{
	GENERATED_BODY()
};


/*
 * Base class of the gesture manager defined in BP
 */

UCLASS(NotPlaceable, Blueprintable, MinimalAPI, meta = (ShowWorldContextPin))
class UE_DEPRECATED(5.5,"This class will be removed from UE5.7") UVPScoutingSubsystemGestureManagerBase : public UObject, public FTickableEditorObject
{
	GENERATED_BODY()

public:
	UVPScoutingSubsystemGestureManagerBase();

	UE_DEPRECATED(5.5,"Code will be removed from UE5.7")
	UFUNCTION(BlueprintNativeEvent, CallInEditor, BlueprintCallable, Category = "Tick", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	void EditorTick(float DeltaSeconds);

	UE_DEPRECATED(5.5,"Code will be removed from UE5.7")
	UFUNCTION(BlueprintNativeEvent, CallInEditor, BlueprintCallable, Category = "VR", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	void OnVREditingModeEnter();
	
	UE_DEPRECATED(5.5,"Code will be removed from UE5.7")
	UFUNCTION(BlueprintNativeEvent, CallInEditor, BlueprintCallable, Category = "VR", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	void OnVREditingModeExit();

	//~ Begin UObject interface
	virtual void BeginDestroy() override;
	//~ End UObject interface

	//~ Begin FTickableEditorObject interface
	virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Conditional; }
	virtual bool IsTickable() const override;
	virtual TStatId GetStatId() const override;
	//~ End FTickableEditorObject interface

private:
	void OnVREditingModeEnterCallback();
	void OnVREditingModeExitCallback();
};


/*
 * Subsystem used for VR Scouting
 */

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCLASS()
class UE_DEPRECATED(5.5,"This class will be removed from UE5.7") UVPScoutingSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UVPScoutingSubsystem();
	
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;

	/** Subsystems can't have any Blueprint implementations, so we attach this class for any BP logic that we to provide. */
	UE_DEPRECATED(5.5,"Property will be removed from UE5.7")
	UPROPERTY(Transient, BlueprintReadOnly, Category = "Virtual Production",  meta=(DeprecatedProperty, DeprecationMessage="This property is deprecated"))
	TObjectPtr<UVPScoutingSubsystemHelpersBase> VPSubsystemHelpers;

	/** GestureManager that manage some user input in VR editor. */
	UE_DEPRECATED(5.5,"Property will be removed from UE5.7")
	UPROPERTY(Transient, BlueprintReadOnly, Category = "Virtual Production", meta=(DeprecatedProperty, DeprecationMessage="This property is deprecated"))
	TObjectPtr<UVPScoutingSubsystemGestureManagerBase> GestureManager;

	/** bool to keep track of whether the settings menu panel in the main menu is open*/
	UE_DEPRECATED(5.5,"Property will be removed from UE5.7")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Main Menu", meta=(DeprecatedProperty, DeprecationMessage="This property is deprecated"))
	bool IsSettingsMenuOpen;

	/** This is a multiplier for grip nav speed so we can keep the grip nav value in the range 0-1 and increase this variable if we need a bigger range */
	UE_DEPRECATED(5.5,"Property will be removed from UE5.7")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Virtual Production", meta=(DeprecatedProperty, DeprecationMessage="This property is deprecated"))
	float GripNavSpeedCoeff;

	// @todo: Guard against user-created name collisions
	/** Open a widget UI in front of the user. Opens default VProd UI (defined via the 'Virtual Scouting User Interface' setting) if null. */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	void ToggleVRScoutingUI(UPARAM(ref) FVREditorFloatingUICreationContext& CreationContext);

	/** Hide VR Sequencer Window*/
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	void HideInfoDisplayPanel();

	/** Check whether a widget UI is open*/
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintPure, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	bool IsVRScoutingUIOpen(const FName& PanelID);

	/** Get UI panel Actor from the passed ID */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintPure, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	AVREditorFloatingUI* GetPanelActor(const FName& PanelID) const;

	/** Get UI panel widget from the passed ID */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintPure, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	UUserWidget* GetPanelWidget(const FName& PanelID) const;

	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static TArray<UVREditorInteractor*> GetActiveEditorVRControllers();

	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintPure, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static const FName GetVProdPanelID(const EVProdPanelIDs Panel)
	{
		switch (Panel)
		{
			case EVProdPanelIDs::Main:
				return VProdPanelID;
			case EVProdPanelIDs::Right:
				return VProdPanelRightID;
			case EVProdPanelIDs::Left:
				return VProdPanelLeftID;
			case EVProdPanelIDs::Context:
				return VProdPanelContextID;
			case EVProdPanelIDs::Timeline:
				return VProdPanelTimelineID;
			case EVProdPanelIDs::Measure:
				return VProdPanelMeasureID;
			case EVProdPanelIDs::Gaffer:
				return VProdPanelGafferID;
		}

		return VProdPanelID;
	};

	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintPure, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static FString GetDirectorName();

	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintPure, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static FString GetShowName();

	
	/** Whether the VR user wants to use the metric system instead of imperial */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintPure, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static bool IsUsingMetricSystem();

	/** Set whether the VR user wants to use the metric system instead of imperial */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static void SetIsUsingMetricSystem(const bool bInUseMetricSystem);
	
	/** Whether the VR user wants to have the transform gizmo enabled */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintPure, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static bool IsUsingTransformGizmo();

	/** Set whether the VR user wants to have the transform gizmo enabled */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static void SetIsUsingTransformGizmo(const bool bInIsUsingTransformGizmo);

	/** Set value of cvar "VI.ShowTransformGizmo" */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static void SetShowTransformGizmoCVar(const bool bInShowTransformGizmoCVar);

	/** Get flight speed for scouting in VR */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintPure, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static float GetFlightSpeed();

	/** Set flight speed for scouting in VR */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static void SetFlightSpeed(const float InFlightSpeed);

	/** Get grip nav speed for scouting in VR */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintPure, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static float GetGripNavSpeed();

	/** Set grip nav speed for scouting in VR */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static void SetGripNavSpeed(const float InGripNavSpeed);

	/** Whether grip nav inertia is enabled when scouting in VR */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintPure, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static bool IsUsingInertiaDamping();

	/** Set whether grip nav inertia is enabled when scouting in VR */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static void SetIsUsingInertiaDamping(const bool bInIsUsingInertiaDamping);

	/** Set value of cvar "VI.HighSpeedInertiaDamping" */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static void SetInertiaDampingCVar(const float InInertiaDamping);

	/** Whether the helper system on the controllers is enabled */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintPure, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static bool IsHelperSystemEnabled();

	/** Set whether the helper system on the controllers is enabled   */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static void SetIsHelperSystemEnabled(const bool bInIsHelperSystemEnabled);

	/** Get VR Editor Mode object */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintPure, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static UVREditorMode* GetVREditorMode();

	/** Enter VR Mode */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static bool EnterVRMode();

	/** Exit VR Mode */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static void ExitVRMode();

	/** Whether location grid snapping is enabled */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static bool IsLocationGridSnappingEnabled();

	/** Toggle location grid snapping */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static void ToggleLocationGridSnapping();

	/** Whether rotation grid snapping is enabled */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static bool IsRotationGridSnappingEnabled();

	/** Toggle rotation grid snapping */
	UE_DEPRECATED(5.5,"Function will be removed from UE5.7")
	UFUNCTION(BlueprintCallable, Category = "Virtual Production", meta=(DeprecatedFunction, DeprecationMessage="This function will be removed from UE5.7"))
	static void ToggleRotationGridSnapping();

private:

	FDelegateHandle EngineInitCompleteDelegate;

	void OnEngineInitComplete();
	
	// Static IDs when submitting open/close requests for the VProd main menu panels. VREditorUISystem uses FNames to manage its panels, so these should be used for consistency.	
	static const FName VProdPanelID;
	static const FName VProdPanelLeftID;
	static const FName VProdPanelRightID;
	static const FName VProdPanelContextID;
	static const FName VProdPanelTimelineID;
	static const FName VProdPanelMeasureID;
	static const FName VProdPanelGafferID;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS