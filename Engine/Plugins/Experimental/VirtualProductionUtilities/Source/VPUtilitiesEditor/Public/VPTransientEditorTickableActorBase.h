// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VPEditorTickableActorBase.h"

#include "VPTransientEditorTickableActorBase.generated.h"

/**
 * Specific VPEditorTickableActor explicitely marked as Transient
 */
UCLASS(Abstract, Transient)
class AVPTransientEditorTickableActorBase : public AVPEditorTickableActorBase
{
	GENERATED_BODY()
};
