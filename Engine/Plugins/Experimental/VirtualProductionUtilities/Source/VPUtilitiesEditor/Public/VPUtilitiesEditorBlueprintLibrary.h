// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "VPEditorTickableActorBase.h"
#include "VPTransientEditorTickableActorBase.h"
#include "VPUtilitiesEditorBlueprintLibrary.generated.h"

class UTexture;

UCLASS()
class VPUTILITIESEDITOR_API UVPUtilitiesEditorBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Spawn an editor-only virtual production tickable actor 
	 * @note Actors based on the non-transient AVPEditorTickableActorBase will be saved in the level. 
	 * @note Being non-transient also means that transactions happening on them will be replicated on other connected multi-user machines
	 */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production")
	static AVPEditorTickableActorBase* SpawnVPEditorTickableActor(UObject* ContextObject, const TSubclassOf<AVPEditorTickableActorBase> ActorClass, const FVector Location, const FRotator Rotation);
		
	/**
	 * Spawn an editor-only transient virtual production tickable actor
	 * @note Actors based on the transient AVPTransientEditorTickableActorBase will NOT be saved in the level.
	 * @note Being transient also means that transactions happening on them will NOT be replicated on other connected multi-user machines
	 */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production")
	static AVPTransientEditorTickableActorBase* SpawnVPTransientEditorTickableActor(UObject* ContextObject, const TSubclassOf<AVPTransientEditorTickableActorBase> ActorClass, const FVector Location, const FRotator Rotation);

	/** Imports Image file into VirtualProduction/Snapshots/ folder */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production")
	static UTexture* ImportSnapshotTexture(FString FileName, FString SubFolderName, FString AbsolutePathPackage);

	/** Get the default OSC server. */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production")
	static class UOSCServer* GetDefaultOSCServer();
};