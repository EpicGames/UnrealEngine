// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"

#include "EditorUtilityWidget.h"
#include "EditorUtilityObject.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"

#include "VPUtilitiesEditorSettings.generated.h"

/**
 * Virtual Production utilities settings for editor
 */
UCLASS(config=VirtualProductionUtilities)
class VPUTILITIESEDITOR_API UVPUtilitiesEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	UVPUtilitiesEditorSettings();

	/** The default user interface that we'll use for virtual scouting */
	UE_DEPRECATED(5.5,"This property  will be removed from UE5.7")
	UPROPERTY(EditAnywhere, config, Category = "Legacy Virtual Scouting", meta = (DisplayName = "Virtual Scouting User Interface", DeprecatedProperty, DeprecationMessage="This property is deprecated"))
	TSoftClassPtr<UEditorUtilityWidget> VirtualScoutingUI;
	
	/** Speed when flying in VR*/
	UE_DEPRECATED(5.5,"This property  will be removed from UE5.7")
	UPROPERTY(EditAnywhere, config, Category = "Legacy Virtual Scouting", meta = (ClampMin = "0.0", ClampMax = "1.0", DisplayName = "Virtual Scouting Flight Speed", DeprecatedProperty, DeprecationMessage="This property is deprecated"))
	float FlightSpeed = 0.5f;
	
	/** Speed when using grip nav in VR */
	UE_DEPRECATED(5.5,"This property  will be removed from UE5.7")
	UPROPERTY(EditAnywhere, config, Category = "Legacy Virtual Scouting", meta = (ClampMin = "0.0", ClampMax = "1.0", DisplayName = "Virtual Scouting Grip Nav Speed", DeprecatedProperty, DeprecationMessage="This property is deprecated"))
	float GripNavSpeed = 0.25f;
	
	/** Whether to use the metric system or imperial for measurements */
	UE_DEPRECATED(5.5,"This property  will be removed from UE5.7")
	UPROPERTY(EditAnywhere, config, Category = "Legacy Virtual Scouting", meta = (DisplayName = "Show Measurements In Metric Units", DeprecatedProperty, DeprecationMessage="This property is deprecated"))
	bool bUseMetric = false;
	
	/** Whether to enable or disable the transform gizmo */
	UE_DEPRECATED(5.5,"This property  will be removed from UE5.7")
	UPROPERTY(EditAnywhere, config, Category = "Legacy Virtual Scouting", meta = (DisplayName = "Enable Transform Gizmo In VR", DeprecatedProperty, DeprecationMessage="This property is deprecated"))
	bool bUseTransformGizmo = false;
	
	/** If true, the user will use inertia damping to stop after grip nav. Otherwise the user will just stop immediately */
	UE_DEPRECATED(5.5,"This property  will be removed from UE5.7")
	UPROPERTY(EditAnywhere, config, Category = "Legacy Virtual Scouting", meta = (DisplayName = "Use Grip Inertia Damping", DeprecatedProperty, DeprecationMessage="This property is deprecated"))
	bool bUseGripInertiaDamping = true;
	
	/** Damping applied to inertia */
	UE_DEPRECATED(5.5,"This property  will be removed from UE5.7")
	UPROPERTY(EditAnywhere, config, Category = "Legacy Virtual Scouting", meta = (ClampMin = "0.0", ClampMax = "1.0", DisplayName = "Inertia Damping", DeprecatedProperty, DeprecationMessage="This property is deprecated"))
	float InertiaDamping = 0.95f;

	/** Whether the helper system on the controllers is enabled */
	UE_DEPRECATED(5.5,"Code will be removed from UE5.7")
	UPROPERTY(EditAnywhere, config, Category = "Legacy Virtual Scouting", meta = (DisplayName = "Helper System Enabled ", DeprecatedProperty, DeprecationMessage="This property is deprecated"))
	bool bIsHelperSystemEnabled = true;
	
	/** When enabled, an OSC server will automatically start on launch. */
	UPROPERTY(config, EditAnywhere, Category = "OSC", DisplayName = "Start an OSC Server when the editor launches")
	bool bStartOSCServerAtLaunch;

	/** The OSC server's address. */
	UPROPERTY(config, EditAnywhere, Category = "OSC", DisplayName = "OSC Server Address")
	FString OSCServerAddress;

	/** The OSC server's port. */
	UPROPERTY(config, EditAnywhere, Category = "OSC", DisplayName = "OSC Server Port")
	uint16 OSCServerPort;

	/** What EditorUtilityObject should be ran on editor launch. */
	UPROPERTY(config, EditAnywhere, Category = "OSC", meta = (AllowedClasses = "/Script/Blutility.EditorUtilityBlueprint", DisplayName = "OSC Listeners"))
	TArray<FSoftObjectPath> StartupOSCListeners;

	/** ScoutingSubsystem class to use for Blueprint helpers */
	UE_DEPRECATED(5.5,"Code will be removed from UE5.7")
	UPROPERTY(config, meta = (MetaClass = "/Script/VPUtilitiesEditor.VPScoutingSubsystemHelpersBase"))
	FSoftClassPath ScoutingSubsystemEditorUtilityClassPath;

	/** GestureManager class to use by the ScoutingSubsystem */
	UE_DEPRECATED(5.5,"Code will be removed from UE5.7")
	UPROPERTY(config, meta = (MetaClass = "/Script/VPUtilitiesEditor.VPScoutingSubsystemGestureManagerBase"))
	FSoftClassPath GestureManagerEditorUtilityClassPath;

	/** GestureManager class to use by the ScoutingSubsystem */
	UE_DEPRECATED(5.5,"Code will be removed from UE5.7")
	UPROPERTY(config)
	TArray<FSoftClassPath> AdditionnalClassToLoad;

	UE_DEPRECATED(5.5,"Code will be removed from UE5.7")
	UPROPERTY(config, meta = (AllowedClasses = "/Script/Engine.StaticMesh"))
	FSoftObjectPath VPSplinePreviewMeshPath;
};

/** Per-User editor settings for Virtual Production utilities. */
UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class UVPUtilitiesEditorUserSettings : public UObject
{
	GENERATED_BODY()

public:
	virtual void PostInitProperties() override
	{
		Super::PostInitProperties();

		FString OverrideAddress;
		uint16 OverridePort;

		if (FParse::Value(FCommandLine::Get(), TEXT("-VPOSCServerAddress="), OverrideAddress))
		{
			OverrideOSCServerAddress = MoveTemp(OverrideAddress);
		}
		
		if (FParse::Value(FCommandLine::Get(), TEXT("-VPOSCServerPort="), OverridePort))
		{
			OverrideOSCServerPort = OverridePort;
		}
	}

	/** 
	 * If provided, this address will override the OSC server address provided in the project settings. 
	 * Can also be specified on the command line through -VPOSCServerAddress=<Address>
	 */
	UPROPERTY(config, EditAnywhere, Category = "OSC", DisplayName = "OSC Server Address")
	FString OverrideOSCServerAddress;

	/**
	 * If provided, this port will override the OSC server port provided in the project settings.
	 * Can also be specified on the command line through -VPOSCServerPort=<Port>
	 */
	UPROPERTY(config, EditAnywhere, Category = "OSC", DisplayName = "OSC Server Port")
	uint16 OverrideOSCServerPort;

};