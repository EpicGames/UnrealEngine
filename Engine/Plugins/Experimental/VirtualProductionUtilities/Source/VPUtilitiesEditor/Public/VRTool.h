// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VPTransientEditorTickableActorBase.h"
#include "VRTool.generated.h"

/**
 * Base class for virtual production VR tools
 */

UCLASS()
class UE_DEPRECATED(5.5,"This class will be removed from UE5.7") AVRTool : public AVPTransientEditorTickableActorBase
{
	GENERATED_BODY()

	AVRTool(const FObjectInitializer& ObjectInitializer);

	virtual void OnConstruction(const FTransform& Transform) override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	/** Returns the current InputComponent on this utility actor. This will be NULL unless bReceivesEditorInput is set to true. */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Input|Editor")
	UInputComponent* GetInputComponent() const
	{
		return EditorOnlyInputComponent.Get();
	}
	
	UFUNCTION(BlueprintSetter, Category = "Input|Editor")
	void SetReceivesEditorInput(bool bInValue);
	
	UFUNCTION(BlueprintGetter, BlueprintPure, Category = "Input|Editor")
	bool GetReceivesEditorInput() const
	{
		return bReceivesEditorInput;
	}

private:

	/** Creates the EditorOnlyInputComponent if it does not already exist and registers all subobject callbacks to it */
	void CreateEditorInput();
	
	/** Removes the EditorOnlyInputComponent from this utility actor */
	void RemoveEditorInput();
	
	UPROPERTY(Transient, DuplicateTransient, meta=(DeprecatedProperty, DeprecationMessage="This is deprecated"))
	TObjectPtr<UInputComponent> EditorOnlyInputComponent;

	/** If set to true, then this actor will be able to recieve input delegate callbacks when in the editor. */
	UPROPERTY(EditAnywhere, Category = "Input|Editor", BlueprintSetter = SetReceivesEditorInput, BlueprintGetter = GetReceivesEditorInput, meta=(DeprecatedProperty, DeprecationMessage="This is deprecated"))
	bool bReceivesEditorInput = false;

};