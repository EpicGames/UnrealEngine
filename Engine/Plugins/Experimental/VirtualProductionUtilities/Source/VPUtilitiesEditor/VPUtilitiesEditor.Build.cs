// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VPUtilitiesEditor : ModuleRules
{
	public VPUtilitiesEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
                "Blutility",
				"Core",
				"CoreUObject",
				"EditorSubsystem",
				"Engine",
				"Slate",
				"SlateCore",
				"VPUtilities",                
				"VREditor",
            }
        );

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
                "CinematicCamera",
				"EditorFramework",
				"EditorStyle",
				"GameplayTags",
				"InputCore",
				"LevelEditor",
				"OSC",
				"PlacementMode",
				"Settings",
				"TimeManagement",
				"UMG",
				"UMGEditor",
				"UnrealEd",
				"VPBookmark",
				"VPRoles",
				"VPSettings",
				"WorkspaceMenuStructure",
            }
		);
	}
}
