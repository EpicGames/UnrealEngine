{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Virtual Production Utilities",
	"Description": "Utility classes and functions for Virtual Production",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "VPUtilities",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "VPUtilitiesEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		},
		{
			"Name": "VPBookmark",
			"Type": "UncookedOnly",
			"LoadingPhase": "PostDefault"
		},
		{
			"Name": "VPBookmarkEditor",
			"Type": "Editor",
			"LoadingPhase": "PostDefault"
		}
	],
	"Plugins": [
		{
			"Name": "MultiUserClient",
			"Enabled": true
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		},
		{
			"Name": "SequencerScripting",
			"Enabled": true
		},
		{
			"Name": "Takes",
			"Enabled": true
		},
		{
			"Name": "OSC",
			"Enabled": true
		},
		{
			"Name": "VPRoles",
			"Enabled": true
		},
		{
			"Name": "VPSettings",
			"Enabled": true
		},
		{
			"Name": "ConsoleVariables",
			"Enabled": true			
		},
		{
			"Name": "XRBase",
			"Enabled": true
		},
		{
			"Name": "CineCameraSceneCapture",
			"Enabled": true
		},
		{
			"Name": "InlineMaterialInstance",
			"Enabled": true
		},
		{
			"Name": "ViewportWidgetOverlay",
			"Enabled": true
		}
	]
}
